// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Styling/SlateTypes.h"

/**
 * Delegate type for session owner filter state changes.
 *
 * The first parameter is the name of the owner that changed its enabled state.
 * The second parameter is the new enabled state.
 */
DECLARE_DELEGATE_TwoParams(FOnSessionBrowserOwnerFilterStateChanged, const FString&, bool);


/**
 * Implements a view model for a session owner filter.
 */
class FSessionBrowserOwnerFilter
	: public TSharedFromThis<FSessionBrowserOwnerFilter>
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InOwnerName The user name of the session owner.
	 * @param InEnabled Whether this filter is enabled.
	 * @param InOnStateChanged A delegate that is executed when the filter's enabled state changed.
	 */
	FSessionBrowserOwnerFilter( const FString& InOwnerName, bool InEnabled, FOnSessionBrowserOwnerFilterStateChanged InOnStateChanged )
		: Enabled(InEnabled)
		, OwnerName(InOwnerName)
		, OnStateChanged(InOnStateChanged)
	{ }

public:

	/**
	 * Enables or disables the filter based on the specified check box state.
	 *
	 * @param CheckState The check box state.
	 */
	void EnableFromCheckState( ECheckBoxState CheckState )
	{
		Enabled = (CheckState == ECheckBoxState::Checked);

		OnStateChanged.ExecuteIfBound(OwnerName, Enabled);
	}

	/**
	 * Gets the filter's owner name.
	 *
	 * @return The user name of the session owner.
	 */
	const FString& GetOwnerName()
	{
		return OwnerName;
	}

	/**
	 * Gets the check box state from the filter's enabled state.
	 *
	 * @return Checked if the filter is enabled, Unchecked otherwise.
	 */
	ECheckBoxState GetCheckStateFromIsEnabled() const
	{
		return (Enabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);
	}

	/**
	 * Checks whether this filter is enabled.
	 *
	 * @return true if the filter is enabled, false otherwise.
	 */
	bool IsEnabled() const
	{
		return Enabled;
	}

private:

	/** Holds a flag indicating whether this filter is enabled. */
	bool Enabled;

	/** Holds the name of the session owner. */
	FString OwnerName;

private:

	/** Holds a delegate that is executed when the filter's enabled state changed. */
	FOnSessionBrowserOwnerFilterStateChanged OnStateChanged;
};


/** Type definition for shared pointers to instances of FSessionBrowserOwnerFilter. */
typedef TSharedPtr<FSessionBrowserOwnerFilter> FSessionBrowserOwnerFilterPtr;

/** Type definition for shared references to instances of FSessionBrowserOwnerFilter. */
typedef TSharedRef<FSessionBrowserOwnerFilter> FSessionBrowserOwnerFilterRef;
