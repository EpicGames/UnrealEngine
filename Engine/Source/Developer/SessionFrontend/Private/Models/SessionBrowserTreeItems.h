// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISessionInstanceInfo.h"
#include "ISessionInfo.h"

enum class ESessionBrowserTreeNodeType
{
	Group,
	Instance,
	Session
};


/**
 * Base class for items in the session tree view.
 */
class FSessionBrowserTreeItem
{
public:

	/** Virtual destructor. */
	virtual ~FSessionBrowserTreeItem() = default;

public:

	/**
	 * Adds a child process item to this item.
	 *
	 * @param The child item to add.
	 * @see ClearChildren, GetChildren
	 */
	void AddChild(const TSharedRef<FSessionBrowserTreeItem>& Child)
	{
		Children.Add(Child);
	}

	/**
	 * Clears the collection of child items.
	 *
	 * @see AddChild, GetChildren
	 */
	void ClearChildren()
	{
		Children.Reset();
	}

	/**
	 * Gets the child items.
	 *
	 * @return Child items.
	 * @see AddChild, ClearChildren, GetParent
	 */
	const TArray<TSharedPtr<FSessionBrowserTreeItem>>& GetChildren()
	{
		return Children;
	}

	/**
	 * Gets the parent item.
	 *
	 * @return Parent item.
	 * @see GetChildren, SetParent
	 */
	const TSharedPtr<FSessionBrowserTreeItem>& GetParent()
	{
		return Parent;
	}

	/**
	 * Sets the parent item.
	 *
	 * @param Node The parent item to set.
	 * @see GetParent
	 */
	void SetParent(const TSharedPtr<FSessionBrowserTreeItem>& Node)
	{
		Parent = Node;
	}

public:

	virtual ESessionBrowserTreeNodeType GetType() = 0;

private:

	/** Holds the child process items. */
	TArray<TSharedPtr<FSessionBrowserTreeItem>> Children;

	/** Holds a pointer to the parent item. */
	TSharedPtr<FSessionBrowserTreeItem> Parent;
};


/**
 * Implements a group item in the session tree view.
 */
class FSessionBrowserGroupTreeItem
	: public FSessionBrowserTreeItem
{
public:

	/** Creates and initializes a new instance. */
	FSessionBrowserGroupTreeItem(const FText& InGroupName, const FText& InToolTipText)
		: GroupName(InGroupName)
		, ToolTipText(InToolTipText)
	{ }

	/** Virtual destructor. */
	virtual ~FSessionBrowserGroupTreeItem() = default;

public:

	/**
	 * Gets the name of the group associated with this item.
	 *
	 * @return Group name.
	 */
	FText GetGroupName() const
	{
		return GroupName;
	}

	/**
	 * Gets the tool tip text of the group associated with this item.
	 *
	 * @return Tool tip text.
	 */
	FText GetToolTipText() const
	{
		return ToolTipText;
	}

public:

	// FSessionBrowserTreeNode interface

	virtual ESessionBrowserTreeNodeType GetType() override
	{
		return ESessionBrowserTreeNodeType::Group;
	}

private:

	/** The name of the group associated with this item. */
	FText GroupName;

	/** The tool tip text. */
	FText ToolTipText;
};


/**
 * Implements a instance item in the session tree view.
 */
class FSessionBrowserInstanceTreeItem
	: public FSessionBrowserTreeItem
{
public:

	/** Creates and initializes a new instance. */
	FSessionBrowserInstanceTreeItem(const TSharedRef<ISessionInstanceInfo>& InInstanceInfo)
		: InstanceInfo(InInstanceInfo)
	{ }

	/** Virtual destructor. */
	virtual ~FSessionBrowserInstanceTreeItem() = default;

public:

	/**
	 * Gets the instance info associated with this item.
	 *
	 * @return Instance info.
	 */
	TSharedPtr<ISessionInstanceInfo> GetInstanceInfo() const
	{
		return InstanceInfo.Pin();
	}

public:

	// FSessionBrowserTreeNode interface

	virtual ESessionBrowserTreeNodeType GetType() override
	{
		return ESessionBrowserTreeNodeType::Instance;
	}

private:

	/** Weak pointer to the instance info associated with this item. */
	TWeakPtr<ISessionInstanceInfo> InstanceInfo;
};


/**
 * Implements a session item in the session tree view.
 */
class FSessionBrowserSessionTreeItem
	: public FSessionBrowserTreeItem
{
public:

	/** Creates and initializes a new instance. */
	FSessionBrowserSessionTreeItem(const TSharedRef<ISessionInfo>& InSessionInfo)
		: SessionInfo(InSessionInfo)
	{ }

	/** Virtual destructor. */
	virtual ~FSessionBrowserSessionTreeItem() = default;

public:

	/**
	 * Gets the session info associated with this item.
	 *
	 * @return Session info.
	 */
	TSharedPtr<ISessionInfo> GetSessionInfo() const
	{
		return SessionInfo.Pin();
	}

public:

	// FSessionBrowserTreeNode interface

	virtual ESessionBrowserTreeNodeType GetType() override
	{
		return ESessionBrowserTreeNodeType::Session;
	}

private:

	/** Weak pointer to the session info associated with this item. */
	TWeakPtr<ISessionInfo> SessionInfo;
};
