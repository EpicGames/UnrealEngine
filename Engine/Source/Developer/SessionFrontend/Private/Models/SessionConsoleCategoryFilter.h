// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateTypes.h"

/**
 * Delegate type for category filter state changes.
 *
 * The first parameter is the name of the category that changed its enabled state.
 * The second parameter is the new enabled state.
 */
DECLARE_DELEGATE_TwoParams(FOnSessionConsoleCategoryFilterStateChanged, const FName&, bool);


/**
 * Implements a view model for a console log category filter.
 */
class FSessionConsoleCategoryFilter
	: public TSharedFromThis<FSessionConsoleCategoryFilter>
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InCategory The filter category.
	 * @param InEnabled Whether this filter is enabled.
	 * @param InOnStateChanged A delegate that is executed when the filter's enabled state changed.
	 */
	FSessionConsoleCategoryFilter( const FName& InCategory, bool InEnabled, FOnSessionConsoleCategoryFilterStateChanged InOnStateChanged )
		: Category(InCategory)
		, Enabled(InEnabled)
		, OnStateChanged(InOnStateChanged)
	{ }

public:

	/**
	 * Enables or disables the filter based on the specified check box state.
	 *
	 * @param CheckState The check box state.
	 */
	void EnableFromCheckState( ECheckBoxState CheckState )
	{
		Enabled = (CheckState == ECheckBoxState::Checked);

		OnStateChanged.ExecuteIfBound(Category, Enabled);
	}

	/**
	 * Gets the filter's category.
	 *
	 * @return The category name.
	 */
	FName GetCategory()
	{
		return Category;
	}

	/**
	 * Gets the check box state from the filter's enabled state.
	 *
	 * @return Checked if the filter is enabled, Unchecked otherwise.
	 */
	ECheckBoxState GetCheckStateFromIsEnabled() const
	{
		return (Enabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);
	}

	/**
	 * Checks whether this filter is enabled.
	 *
	 * @return true if the filter is enabled, false otherwise.
	 */
	bool IsEnabled() const
	{
		return Enabled;
	}

private:

	/** Holds the filter's category. */
	FName Category;

	/** Holds a flag indicating whether this filter is enabled. */
	bool Enabled;

private:

	/** Holds a delegate that is executed when the filter's enabled state changed. */
	FOnSessionConsoleCategoryFilterStateChanged OnStateChanged;
};


/** Type definition for shared pointers to instances of FSessionConsoleCategoryFilter. */
typedef TSharedPtr<FSessionConsoleCategoryFilter> FSessionConsoleCategoryFilterPtr;

/** Type definition for shared references to instances of FSessionConsoleCategoryFilter. */
typedef TSharedRef<FSessionConsoleCategoryFilter> FSessionConsoleCategoryFilterRef;
