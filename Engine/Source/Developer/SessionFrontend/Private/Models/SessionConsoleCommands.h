// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/InputChord.h"
#include "Styling/AppStyle.h"
#include "Framework/Commands/Commands.h"
#include "SessionFrontendStyle.h"

#define LOCTEXT_NAMESPACE "SessionConsoleCommands"

/**
 * The device details commands.
 */
class FSessionConsoleCommands
	: public TCommands<FSessionConsoleCommands>
{
public:

	/** Default constructor. */
	FSessionConsoleCommands()
		: TCommands<FSessionConsoleCommands>(
			"SessionConsole",
			NSLOCTEXT("Contexts", "SessionConsole", "Session Console"),
			NAME_None, FSessionFrontendStyle::GetStyleSetName()
		)
	{ }

public:

	// TCommands interface

	virtual void RegisterCommands() override
	{
		UI_COMMAND(Clear, "Clear Log", "Clear the log window", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(SessionCopy, "Copy", "Copy the selected log messages to the clipboard", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(SessionSave, "Save Log...", "Save the entire log to a file", EUserInterfaceActionType::ToggleButton, FInputChord());
	}

public:

	TSharedPtr<FUICommandInfo> Clear;
	TSharedPtr<FUICommandInfo> SessionCopy;
	TSharedPtr<FUICommandInfo> SessionSave;
};

#undef LOCTEXT_NAMESPACE
