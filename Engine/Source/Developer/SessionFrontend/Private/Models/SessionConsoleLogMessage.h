// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Misc/DateTime.h"
#include "IMessageContext.h"

/**
 * Structure for log messages that are displayed in the session console panel.
 */
struct FSessionConsoleLogMessage
{
	/** Holds the log category. */
	FName Category;

	/** Holds the name of the sender's engine instance. */
	FString InstanceName;

	/** Holds the sender's address. */
	FMessageAddress Sender;

	/** Holds the message text. */
	FString Text;

	/** Holds the time at which the message was generated. */
	FDateTime Time;

	/** Holds the number of seconds from the start of the instance at which the message was generated. */
	double TimeSeconds;

	/** Holds the verbosity type. */
	ELogVerbosity::Type Verbosity;

public:

	/**
	 * Creates and initializes a new instance.
	 */
	FSessionConsoleLogMessage(const FMessageAddress& InSender, const FString& InInstanceName, float InTimeSeconds, const FString& InText, ELogVerbosity::Type InVerbosity, const FName& InCategory)
		: Category(InCategory)
		, InstanceName(InInstanceName)
		, Sender(InSender)
		, Text((InCategory != NAME_None) ? InCategory.ToString() + TEXT(": ") + InText : InText)
		, Time(FDateTime::Now())
		, TimeSeconds(InTimeSeconds)
		, Verbosity(InVerbosity)
	{ }
};
