// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateTypes.h"

/**
 * Delegate type for verbosity filter state changes.
 *
 * The first parameter is the verbosity level that changed its enabled state.
 * The second parameter is the new enabled state.
 */
DECLARE_DELEGATE_TwoParams(FOnSessionConsoleVerbosityFilterStateChanged, ELogVerbosity::Type, bool);


/**
 * Implements a view model for a console log category filter.
 */
class FSessionConsoleVerbosityFilter
	: public TSharedFromThis<FSessionConsoleVerbosityFilter>
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InVerbosity The log verbosity level to filter.
	 * @param InIcon The filter's icon.
	 * @param InEnabled Whether this filter is enabled.
	 * @param InName The filter's name.
	 * @param InOnStateChanged A delegate that is executed when the filter's enabled state changed.
	 */
	FSessionConsoleVerbosityFilter( ELogVerbosity::Type InVerbosity, const FSlateBrush* InIcon, bool InEnabled, const FString& InName, FOnSessionConsoleVerbosityFilterStateChanged InOnStateChanged)
		: Enabled(InEnabled)
		, Icon(InIcon)
		, Name(InName)
		, Verbosity(InVerbosity)
		, OnStateChanged(InOnStateChanged)
	{ }

public:

	/**
	 * Enables or disables the filter based on the specified check box state.
	 *
	 * @param CheckState The check box state.
	 */
	void EnableFromCheckState( ECheckBoxState CheckState )
	{
		Enabled = (CheckState == ECheckBoxState::Checked);

		OnStateChanged.ExecuteIfBound(Verbosity, Enabled);
	}

	/**
	 * Gets the check box state from the filter's enabled state.
	 *
	 * @return Checked if the filter is enabled, Unchecked otherwise.
	 */
	ECheckBoxState GetCheckStateFromIsEnabled() const
	{
		return (Enabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);
	}

	/**
	 * Gets the filter's icon.
	 *
	 * @return Filter icon image.
	 */
	const FSlateBrush* GetIcon() const
	{
		return Icon;
	}

	/**
	 * Gets the filter's name.
	 *
	 * @return The name string.
	 */
	const FString& GetName() const
	{
		return Name;
	}

	/**
	 * Gets the filter verbosity level.
	 *
	 * @return Verbosity level.
	 */
	ELogVerbosity::Type GetVerbosity() const
	{
		return Verbosity;
	}

	/**
	 * Checks whether this filter is enabled.
	 *
	 * @return true if the filter is enabled, false otherwise.
	 */
	bool IsEnabled() const
	{
		return Enabled;
	}

private:

	/** Holds a flag indicating whether this filter is enabled. */
	bool Enabled;

	/** Holds the icon image. */
	const FSlateBrush* Icon;

	/** Holds the filter's name. */
	FString Name;

	/** Holds the filter's verbosity level. */
	ELogVerbosity::Type Verbosity;

private:

	/** Holds a delegate that is executed when the filter's enabled state changed. */
	FOnSessionConsoleVerbosityFilterStateChanged OnStateChanged;
};


/** Type definition for shared pointers to instances of FSessionConsoleVerbosityFilter. */
typedef TSharedPtr<FSessionConsoleVerbosityFilter> FSessionConsoleVerbosityFilterPtr;

/** Type definition for shared references to instances of FSessionConsoleVerbosityFilter. */
typedef TSharedRef<FSessionConsoleVerbosityFilter> FSessionConsoleVerbosityFilterRef;
