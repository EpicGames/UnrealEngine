// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Textures/SlateIcon.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "ISessionManager.h"
#include "Widgets/Browser/SSessionBrowser.h"
#include "Widgets/Console/SSessionConsole.h"
#include "Widgets/SSessionFrontend.h"
#include "ISessionFrontendModule.h"
#include "SessionFrontendStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"


static const FName SessionFrontendTabName("SessionFrontend");


/**
 * Implements the SessionFrontend module.
 */
class FSessionFrontendModule
	: public ISessionFrontendModule
{
public:

	// ISessionFrontendModule interface
	
	virtual TSharedRef<class SWidget> CreateSessionBrowser( const TSharedRef<ISessionManager>& SessionManager ) override
	{
		return SNew(SSessionBrowser, SessionManager);
	}
	
	virtual TSharedRef<class SWidget> CreateSessionConsole( const TSharedRef<ISessionManager>& SessionManager ) override
	{
		return SNew(SSessionConsole, SessionManager);
	}

	virtual void InvokeSessionFrontend(FName SubTabToActivate = NAME_None) override
	{
		FGlobalTabmanager::Get()->TryInvokeTab(SessionFrontendTabName);
		if ( WeakFrontend.IsValid() )
		{
			if ( SubTabToActivate != NAME_None )
			{
				WeakFrontend.Pin()->GetTabManager()->TryInvokeTab(SubTabToActivate);
			}
		}
	}
	
public:

	// IModuleInterface interface
	
	virtual void StartupModule() override
	{
		FSessionFrontendStyle::Initialize();

		auto& TabSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(SessionFrontendTabName, FOnSpawnTab::CreateRaw(this, &FSessionFrontendModule::SpawnSessionFrontendTab))
			.SetDisplayName(NSLOCTEXT("FSessionFrontendModule", "FrontendTabTitle", "Session Frontend"))
			.SetTooltipText(NSLOCTEXT("FSessionFrontendModule", "FrontendTooltipText", "Open the Session Frontend tab."))
			.SetIcon(FSlateIcon(FSessionFrontendStyle::GetStyleSetName(), "SessionFrontEnd.TabIcon"));

		TabSpawnerEntry.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory());
	}

	virtual void ShutdownModule() override
	{
		FSessionFrontendStyle::Shutdown();

		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(SessionFrontendTabName);
	}

private:

	/**
	 * Creates a new session front-end tab.
	 *
	 * @param SpawnTabArgs The arguments for the tab to spawn.
	 * @return The spawned tab.
	 */
	TSharedRef<SDockTab> SpawnSessionFrontendTab( const FSpawnTabArgs& SpawnTabArgs )
	{
		const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
			.TabRole(ETabRole::MajorTab);

		TSharedRef<SSessionFrontend> Frontend = SNew(SSessionFrontend, DockTab, SpawnTabArgs.GetOwnerWindow());
		WeakFrontend = Frontend;

		DockTab->SetContent(Frontend);

		return DockTab;
	}

private:
	TWeakPtr<SSessionFrontend> WeakFrontend;
};


IMPLEMENT_MODULE(FSessionFrontendModule, SessionFrontend);
