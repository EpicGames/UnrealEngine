// Copyright Epic Games, Inc. All Rights Reserved.

#include "SessionFrontendStyle.h"

#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateNoResource.h"
#include "Misc/Paths.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/ToolBarStyle.h"

// This is to fix the issue that SlateStyleMacros like IMAGE_BRUSH look for RootToContentDir but StyleSet->RootToContentDir is how this style is set up
#define RootToContentDir StyleSet->RootToContentDir
#define RootToCoreContentDir StyleSet->RootToCoreContentDir

TSharedPtr< FSlateStyleSet > FSessionFrontendStyle::StyleSet = nullptr;

// Const icon sizes
static const FVector2D Icon8x8(8.0f, 8.0f);
static const FVector2D Icon9x19(9.0f, 19.0f);
static const FVector2D Icon14x14(14.0f, 14.0f);
static const FVector2D Icon16x16(16.0f, 16.0f);
static const FVector2D Icon20x20(20.0f, 20.0f);
static const FVector2D Icon22x22(22.0f, 22.0f);
static const FVector2D Icon24x24(24.0f, 24.0f);
static const FVector2D Icon28x28(28.0f, 28.0f);
static const FVector2D Icon27x31(27.0f, 31.0f);
static const FVector2D Icon26x26(26.0f, 26.0f);
static const FVector2D Icon32x32(32.0f, 32.0f);
static const FVector2D Icon40x40(40.0f, 40.0f);
static const FVector2D Icon48x48(48.0f, 48.0f);
static const FVector2D Icon75x82(75.0f, 82.0f);
static const FVector2D Icon360x32(360.0f, 32.0f);
static const FVector2D Icon171x39(171.0f, 39.0f);
static const FVector2D Icon170x50(170.0f, 50.0f);
static const FVector2D Icon267x140(170.0f, 50.0f);

void FSessionFrontendStyle::Initialize()
{
	LLM_SCOPE_BYNAME(TEXT("SessionFrontend"));

	// Only register once
	if( StyleSet.IsValid() )
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet("SessionFrontendStyle") );
	
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Session Console tab
	{
		StyleSet->Set("SessionConsole.SessionCopy", new IMAGE_BRUSH("Icons/icon_file_open_40x", Icon40x40));
		StyleSet->Set("SessionConsole.SessionCopy.Small", new IMAGE_BRUSH("Icons/icon_file_open_16px", Icon20x20));
		StyleSet->Set("SessionConsole.Clear", new IMAGE_BRUSH("Icons/icon_file_new_40x", Icon40x40));
		StyleSet->Set("SessionConsole.Clear.Small", new IMAGE_BRUSH("Icons/icon_file_new_16px", Icon20x20));
		StyleSet->Set("SessionConsole.SessionSave", new IMAGE_BRUSH("Icons/icon_file_savelevels_40x", Icon40x40));
		StyleSet->Set("SessionConsole.SessionSave.Small", new IMAGE_BRUSH("Icons/icon_file_savelevels_16px", Icon20x20));
	}

	// Session Frontend Window
	{
		StyleSet->Set("SessionFrontEnd.TabIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/SessionFrontend", Icon16x16));
		StyleSet->Set("SessionFrontEnd.Tabs.Tools", new CORE_IMAGE_BRUSH("/Icons/icon_tab_Tools_16x", Icon16x16));
		StyleSet->Set("SessionFrontEnd.Tabs.Console", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Console", Icon16x16));
		StyleSet->Set("SessionFrontEnd.Tabs.Automation", new CORE_IMAGE_BRUSH_SVG("Starship/Common/AutomationTools", Icon16x16));
		StyleSet->Set("SessionFrontEnd.Tabs.ScreenComparison", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Monitor", Icon16x16));
		StyleSet->Set("SessionFrontEnd.Tabs.TraceControl", new IMAGE_BRUSH_SVG("Starship/Common/TraceDataFiltering", Icon16x16));
	}

	// Sesssion Browser
	{
		StyleSet->Set("SessionBrowser.Row.Name", new IMAGE_BRUSH("ContentBrowser/FilterChecked", FVector2D(7.0f, 24.0f)));
	}
	
	FSlateStyleRegistry::RegisterSlateStyle( *StyleSet.Get() );
}

void FSessionFrontendStyle::Shutdown()
{
	if( StyleSet.IsValid() )
	{
		FSlateStyleRegistry::UnRegisterSlateStyle( *StyleSet.Get() );
		ensure( StyleSet.IsUnique() );
		StyleSet.Reset();
	}
}

const ISlateStyle& FSessionFrontendStyle::Get()
{
	return *( StyleSet.Get() );
}

const FName& FSessionFrontendStyle::GetStyleSetName()
{
	return StyleSet->GetStyleSetName();
}