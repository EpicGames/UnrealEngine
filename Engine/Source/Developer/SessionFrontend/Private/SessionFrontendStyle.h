// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FSessionFrontendStyle
{
public:
	static void Initialize();
	static void Shutdown();

	static const ISlateStyle& Get();
	static const FName& GetStyleSetName();
	static const FSlateBrush* GetBrush(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return StyleSet->GetBrush(PropertyName, Specifier);
	}

private:
	/** Singleton instances of this style. */
	static TSharedPtr<class FSlateStyleSet> StyleSet;	
};