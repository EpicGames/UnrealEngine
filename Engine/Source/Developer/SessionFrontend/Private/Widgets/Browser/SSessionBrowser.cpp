// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Browser/SSessionBrowser.h"
#include "Misc/MessageDialog.h"
#include "Misc/App.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Models/SessionBrowserTreeItems.h"
#include "Widgets/Browser/SSessionBrowserTreeGroupRow.h"
#include "Widgets/Browser/SSessionBrowserTreeInstanceRow.h"
#include "Widgets/Browser/SSessionBrowserTreeSessionRow.h"

#define LOCTEXT_NAMESPACE "SSessionBrowser"


/* SSessionBrowser structors
 *****************************************************************************/

SSessionBrowser::~SSessionBrowser()
{
	if (SessionManager.IsValid())
	{
		for (auto& SessionInfo : AvailableSessions)
		{
			SessionInfo->OnInstanceDiscovered().RemoveAll(this);
		}
		SessionManager->OnInstanceSelectionChanged().RemoveAll(this);
		SessionManager->OnSelectedSessionChanged().RemoveAll(this);
		SessionManager->OnSessionsUpdated().RemoveAll(this);
	}
}


/* SSessionBrowser interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SSessionBrowser::Construct( const FArguments& InArgs, TSharedRef<ISessionManager> InSessionManager )
{
	IgnoreSessionManagerEvents = false;
	updatingTreeExpansion = false;
	bCanSetDefaultSelection = true;
	SessionManager = InSessionManager;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
				// session tree
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(0.0f)
				[
					SAssignNew(SessionTreeView, STreeView<TSharedPtr<FSessionBrowserTreeItem>>)
					.OnExpansionChanged(this, &SSessionBrowser::HandleSessionTreeViewExpansionChanged)
					.OnGenerateRow(this, &SSessionBrowser::HandleSessionTreeViewGenerateRow)
					.OnGetChildren(this, &SSessionBrowser::HandleSessionTreeViewGetChildren)
					.OnSelectionChanged(this, &SSessionBrowser::HandleSessionTreeViewSelectionChanged)
					.SelectionMode(ESelectionMode::Single)
					.TreeItemsSource(&SessionTreeItems)
					.HeaderRow
					(
						SNew(SHeaderRow)

						+ SHeaderRow::Column("Name")
						.DefaultLabel(LOCTEXT("InstanceListNameColumnHeader", "Name"))
						.FillWidth(0.4f)

						+ SHeaderRow::Column("Type")
						.DefaultLabel(LOCTEXT("InstanceListTypeColumnHeader", "Type"))
						.FillWidth(0.15f)

						+ SHeaderRow::Column("Device")
						.DefaultLabel(LOCTEXT("InstanceListDeviceColumnHeader", "Device"))
						.FillWidth(0.3f)

						+ SHeaderRow::Column("Status")
						.DefaultLabel(LOCTEXT("InstanceListStatusColumnHeader", "Status"))
						.FillWidth(0.15f)
						.HAlignCell(HAlign_Right)
						.HAlignHeader(HAlign_Right)
					)
				]
			]

			+ SOverlay::Slot()
			.VAlign(EVerticalAlignment::VAlign_Bottom)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.Padding(FMargin(3.0f, 0.0f, 3.0f, 5.0f))
			[
				SNew(STextBlock)
				.AutoWrapText(true)
				.ColorAndOpacity(FSlateColor(EStyleColor::AccentGray))
				.Visibility_Lambda([this]() { return ItemMap.Num() < 10 ? EVisibility::Visible : EVisibility::Collapsed; })
				.Text(LOCTEXT("MessagingInfo", "Sessions must be launched with the \"-messaging\" argument to be visible."))
			]
		]
	];

	AppGroupItem = MakeShareable(new FSessionBrowserGroupTreeItem(LOCTEXT("AppGroupName", "This Application"), LOCTEXT("AppGroupToolTip", "The application instance that this session browser belongs to")));
	OtherGroupItem = MakeShareable(new FSessionBrowserGroupTreeItem(LOCTEXT("OtherGroupName", "Other Sessions"), LOCTEXT("OtherGroupToolTip", "All sessions that belong to other users")));
	OwnerGroupItem = MakeShareable(new FSessionBrowserGroupTreeItem(LOCTEXT("OwnerGroupName", "My Sessions"), LOCTEXT("OwnerGroupToolTip", "All sessions that were started by you")));
	StandaloneGroupItem = MakeShareable(new FSessionBrowserGroupTreeItem(LOCTEXT("StandaloneGroupName", "Standalone Instances"), LOCTEXT("StandaloneGroupToolTip", "Engine instances that don't belong to any particular session")));

	SessionTreeItems.Add(AppGroupItem);
	SessionTreeItems.Add(OwnerGroupItem);
	SessionTreeItems.Add(OtherGroupItem);
	SessionTreeItems.Add(StandaloneGroupItem);

	SessionManager->OnInstanceSelectionChanged().AddSP(this, &SSessionBrowser::HandleSessionManagerInstanceSelectionChanged);
	SessionManager->OnSelectedSessionChanged().AddSP(this, &SSessionBrowser::HandleSessionManagerSelectedSessionChanged);
	SessionManager->OnSessionsUpdated().AddSP(this, &SSessionBrowser::HandleSessionManagerSessionsUpdated);

	ReloadSessions();

	updatingTreeExpansion = true;
	SessionTreeView->SetSingleExpandedItem(AppGroupItem);
	updatingTreeExpansion = false;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SSessionBrowser implementation
 *****************************************************************************/

void SSessionBrowser::ExpandItem(const TSharedPtr<FSessionBrowserTreeItem>& Item)
{
	SessionTreeView->SetItemExpansion(Item, true);

	if (Item.IsValid())
	{
		const auto& ParentItem = Item->GetParent();

		if (ParentItem.IsValid())
		{
			SessionTreeView->SetItemExpansion(ParentItem, true);
		}
	}
}


void SSessionBrowser::FilterSessions()
{
	// clear tree groups
	AppGroupItem->ClearChildren();
	OwnerGroupItem->ClearChildren();
	OtherGroupItem->ClearChildren();
	StandaloneGroupItem->ClearChildren();

	// update tree items
	TMap<FGuid, TSharedPtr<FSessionBrowserTreeItem>> NewItemMap;

	for (auto& SessionInfo : AvailableSessions)
	{
		// process session
		bool LocalOwner = SessionInfo->GetSessionOwner() == FPlatformProcess::UserName(false);

		if (!SessionInfo->IsStandalone() && !LocalOwner)
		{
			continue;
		}

		TSharedPtr<FSessionBrowserTreeItem> SessionItem = ItemMap.FindRef(SessionInfo->GetSessionId());

		if (SessionItem.IsValid())
		{
			SessionItem->ClearChildren();
		}
		else
		{
			SessionItem = MakeShareable(new FSessionBrowserSessionTreeItem(SessionInfo.ToSharedRef()));
		}

		NewItemMap.Add(SessionInfo->GetSessionId(), SessionItem);

		// add session to group
		TArray<TSharedPtr<ISessionInstanceInfo>> Instances;
		SessionInfo->GetInstances(Instances);

		if (LocalOwner)
		{
			if (!FApp::IsThisInstance(Instances[0]->GetInstanceId()))
			{
				OwnerGroupItem->AddChild(SessionItem.ToSharedRef());
				SessionItem->SetParent(OwnerGroupItem);
			}
		}
		else if (SessionInfo->IsStandalone())
		{
			StandaloneGroupItem->AddChild(SessionItem.ToSharedRef());
			SessionItem->SetParent(StandaloneGroupItem);
		}
		else
		{
			OtherGroupItem->AddChild(SessionItem.ToSharedRef());
			SessionItem->SetParent(OtherGroupItem);
		}
		
		// process session instances
		for (auto& InstanceInfo : Instances)
		{
			TSharedPtr<FSessionBrowserTreeItem> InstanceItem = ItemMap.FindRef(InstanceInfo->GetInstanceId());

			if (!InstanceItem.IsValid())
			{
				InstanceItem = MakeShareable(new FSessionBrowserInstanceTreeItem(InstanceInfo.ToSharedRef()));
			}

			AddInstanceItemToTree(SessionItem, InstanceItem, InstanceInfo);

			NewItemMap.Add(InstanceInfo->GetInstanceId(), InstanceItem);
		}
	}

	ItemMap = NewItemMap;

	// refresh tree view
	SessionTreeView->RequestTreeRefresh();

	if ( bCanSetDefaultSelection && SessionTreeView->GetNumItemsSelected() == 0 && ThisAppInstance.IsValid() && !FApp::GetSessionName().Equals("UnrealInsights"))
	{
		bCanSetDefaultSelection = false;
		SessionTreeView->SetItemSelection(ThisAppInstance.Pin(), true, ESelectInfo::Direct);
	}
}

 void SSessionBrowser::AddInstanceItemToTree(TSharedPtr<FSessionBrowserTreeItem>& SessionItem, const TSharedPtr<FSessionBrowserTreeItem>& InstanceItem, const TSharedPtr<ISessionInstanceInfo>& InstanceInfo)
{
	// add instance to group or session
	if (FApp::IsThisInstance(InstanceInfo->GetInstanceId()))
	{
		AppGroupItem->AddChild(InstanceItem.ToSharedRef());
		InstanceItem->SetParent(AppGroupItem);

		ThisAppInstance = InstanceItem;
	}
	else
	{
		InstanceItem->SetParent(SessionItem);
		SessionItem->AddChild(InstanceItem.ToSharedRef());
	}
}

void SSessionBrowser::ReloadSessions()
{
	for (auto& SessionInfo : AvailableSessions)
	{
		SessionInfo->OnInstanceDiscovered().RemoveAll(this);
	}

	SessionManager->GetSessions(AvailableSessions);

	for (auto& SessionInfo : AvailableSessions)
	{
		SessionInfo->OnInstanceDiscovered().AddSP(this, &SSessionBrowser::HandleSessionManagerInstanceDiscovered);
	}

	FilterSessions();
}


/* SSessionBrowser event handlers
 *****************************************************************************/

void SSessionBrowser::HandleSessionManagerInstanceSelectionChanged(const TSharedPtr<ISessionInstanceInfo>& Instance, bool Selected)
{
	if (IgnoreSessionManagerEvents)
	{
		return;
	}

	const auto& InstanceItem = ItemMap.FindRef(Instance->GetInstanceId());

	if (InstanceItem.IsValid())
	{
		SessionTreeView->ClearSelection();
		SessionTreeView->SetItemSelection(InstanceItem, Selected);
	}
}


void SSessionBrowser::HandleSessionManagerSelectedSessionChanged(const TSharedPtr<ISessionInfo>& SelectedSession)
{
	if (IgnoreSessionManagerEvents)
	{
		return;
	}

	updatingTreeExpansion = true;
	{
		if (SelectedSession.IsValid())
		{
			ExpandItem(ItemMap.FindRef(SelectedSession->GetSessionId()));
		}
		else
		{
			SessionTreeView->SetSingleExpandedItem(nullptr);
		}
	}
	updatingTreeExpansion = false;
}

void SSessionBrowser::HandleSessionManagerInstanceDiscovered(const TSharedRef<ISessionInfo>& OwnerSession, const TSharedRef<ISessionInstanceInfo>& DiscoveredInstance)
{
	TSharedPtr<FSessionBrowserTreeItem> SessionItem = ItemMap.FindRef(OwnerSession->GetSessionId());
	if (SessionItem.IsValid())
	{
		// add the item if it's not already there
		TSharedPtr<FSessionBrowserTreeItem> InstanceItem = ItemMap.FindRef(DiscoveredInstance->GetInstanceId());
		if (!InstanceItem.IsValid())
		{
			InstanceItem = MakeShareable(new FSessionBrowserInstanceTreeItem(DiscoveredInstance));

			AddInstanceItemToTree(SessionItem, InstanceItem, DiscoveredInstance);
			ItemMap.Add(DiscoveredInstance->GetInstanceId(), InstanceItem);

			// refresh tree view
			SessionTreeView->RequestTreeRefresh();
		}
	}
}

void SSessionBrowser::HandleSessionManagerSessionsUpdated()
{
	ReloadSessions();
}

FText SSessionBrowser::HandleSessionTreeRowGetToolTipText(TSharedPtr<FSessionBrowserTreeItem> Item) const
{
	FTextBuilder ToolTipTextBuilder;

	if (Item->GetType() == ESessionBrowserTreeNodeType::Instance)
	{
		TSharedPtr<ISessionInstanceInfo> InstanceInfo = StaticCastSharedPtr<FSessionBrowserInstanceTreeItem>(Item)->GetInstanceInfo();

		if (InstanceInfo.IsValid())
		{
			const FCoreTexts& CoreTexts = FCoreTexts::Get();
			const FText PlayHasBegunText = InstanceInfo->PlayHasBegun() ? CoreTexts.Yes : CoreTexts.No;

			ToolTipTextBuilder.AppendLineFormat(LOCTEXT("InstanceToolTipInstanceId", "Instance ID: {0}"), FText::FromString(InstanceInfo->GetInstanceId().ToString(EGuidFormats::DigitsWithHyphensInBraces)));
			ToolTipTextBuilder.AppendLine();
			ToolTipTextBuilder.AppendLineFormat(LOCTEXT("InstanceToolTipBuildDate", "Build Date: {0}"), FText::FromString(InstanceInfo->GetBuildDate()));
			ToolTipTextBuilder.AppendLineFormat(LOCTEXT("InstanceToolTipEngineVersion", "Engine Version: {0}"), InstanceInfo->GetEngineVersion() == 0 ? LOCTEXT("CustomBuildVersion", "Custom Build") : FText::FromString(FString::FromInt(InstanceInfo->GetEngineVersion())));
			ToolTipTextBuilder.AppendLineFormat(LOCTEXT("InstanceToolTipPlatform", "Platform: {0}"), FText::FromString(InstanceInfo->GetPlatformName()));
			ToolTipTextBuilder.AppendLineFormat(LOCTEXT("InstanceToolTipCurrentLevel", "Current Level: {0}"), FText::FromString(InstanceInfo->GetCurrentLevel()));
			ToolTipTextBuilder.AppendLineFormat(LOCTEXT("InstanceToolTipWorldTimeSeconds", "World Time: {0}"), FText::AsTimespan(FTimespan::FromSeconds(InstanceInfo->GetWorldTimeSeconds())));
			ToolTipTextBuilder.AppendLineFormat(LOCTEXT("InstanceToolTipPlayBegun", "Play Has Begun: {0}"), PlayHasBegunText);
			ToolTipTextBuilder.AppendLine();
			ToolTipTextBuilder.AppendLineFormat(LOCTEXT("SessionToolTipLastUpdateTime", "Last Update Time: {0}"), FText::AsDateTime(InstanceInfo->GetLastUpdateTime()));
		}
	}

	return ToolTipTextBuilder.ToText();
}


void SSessionBrowser::HandleSessionTreeViewExpansionChanged(TSharedPtr<FSessionBrowserTreeItem> TreeItem, bool bIsExpanded)
{
	if ( updatingTreeExpansion || !TreeItem.IsValid())
	{
		return;
	}

	if (TreeItem->GetType() == ESessionBrowserTreeNodeType::Instance)
	{
		return;
	}

	IgnoreSessionManagerEvents = true;
	{
		if (bIsExpanded)
		{
			updatingTreeExpansion = true;
			{
				ExpandItem(TreeItem);
			}
			updatingTreeExpansion = false;

			// select session
			if (TreeItem->GetType() == ESessionBrowserTreeNodeType::Session)
			{
				SessionManager->SelectSession(StaticCastSharedPtr<FSessionBrowserSessionTreeItem>(TreeItem)->GetSessionInfo());
			}
			else
			{
				SessionManager->SelectSession(nullptr);
			}
		}
		else if (TreeItem->GetType() != ESessionBrowserTreeNodeType::Instance)
		{
			// deselect session
			SessionManager->SelectSession(nullptr);
		}
	}
	IgnoreSessionManagerEvents = false;
}


TSharedRef<ITableRow> SSessionBrowser::HandleSessionTreeViewGenerateRow(TSharedPtr<FSessionBrowserTreeItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (Item->GetType() == ESessionBrowserTreeNodeType::Group)
	{
		return SNew(SSessionBrowserTreeGroupRow, OwnerTable)
			.Item(StaticCastSharedRef<FSessionBrowserGroupTreeItem>(Item.ToSharedRef()));
	}

	if (Item->GetType() == ESessionBrowserTreeNodeType::Session)
	{
		return SNew(SSessionBrowserTreeSessionRow, OwnerTable)
			.Item(StaticCastSharedRef<FSessionBrowserSessionTreeItem>(Item.ToSharedRef()));
	}

	return SNew(SSessionBrowserTreeInstanceRow, OwnerTable)
		.Item(StaticCastSharedRef<FSessionBrowserInstanceTreeItem>(Item.ToSharedRef()))
		.ToolTipText(this, &SSessionBrowser::HandleSessionTreeRowGetToolTipText, Item)
		.ShowSelection(true);
}


void SSessionBrowser::HandleSessionTreeViewGetChildren(TSharedPtr<FSessionBrowserTreeItem> Item, TArray<TSharedPtr<FSessionBrowserTreeItem>>& OutChildren)
{
	if (Item.IsValid())
	{
		OutChildren = Item->GetChildren();
	}
}


void SSessionBrowser::HandleSessionTreeViewSelectionChanged(const TSharedPtr<FSessionBrowserTreeItem> Item, ESelectInfo::Type SelectInfo)
{
	IgnoreSessionManagerEvents = true;
	{
		if (Item.IsValid())
		{
			if (Item->GetType() == ESessionBrowserTreeNodeType::Instance)
			{
				const auto& InstanceInfo = StaticCastSharedPtr<FSessionBrowserInstanceTreeItem>(Item)->GetInstanceInfo();

				if (InstanceInfo.IsValid())
				{
					SessionManager->SelectSession(InstanceInfo->GetOwnerSession());

					SessionManager->SetInstanceSelected(InstanceInfo.ToSharedRef(), true);
				}
			}
		}

		{
			// check if any instances are no longer selected
			TArray<TSharedPtr<ISessionInstanceInfo>> UnselectedSessions;
			for (const auto& InstanceInfo : SessionManager->GetSelectedInstances())
			{
				const TSharedPtr<FSessionBrowserTreeItem>& InstanceItem = ItemMap.FindRef(InstanceInfo->GetInstanceId());

				if (!SessionTreeView->IsItemSelected(InstanceItem))
				{
					UnselectedSessions.Add(InstanceInfo);
				}
			}

			for (const auto& Session : UnselectedSessions)
			{
				SessionManager->SetInstanceSelected(Session.ToSharedRef(), false);
			}
		}
	}
	IgnoreSessionManagerEvents = false;
}


FReply SSessionBrowser::HandleTerminateSessionButtonClicked()
{
	int32 DialogResult = FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("TerminateSessionDialogPrompt", "Are you sure you want to terminate this session and its instances?"));

	if (DialogResult == EAppReturnType::Yes)
	{
		const TSharedPtr<ISessionInfo>& SelectedSession = SessionManager->GetSelectedSession();

		if (SelectedSession.IsValid())
		{
			if (SelectedSession->GetSessionOwner() == FPlatformProcess::UserName(false))
			{
				SelectedSession->Terminate();
			}
			else
			{
				FMessageDialog::Open( EAppMsgType::Ok, FText::Format( LOCTEXT("TerminateDeniedPrompt", "You are not authorized to terminate the currently selected session, because it is owned by {0}"), FText::FromString( SelectedSession->GetSessionOwner() ) ) );
			}
		}
	}

	return FReply::Handled();
}


bool SSessionBrowser::HandleTerminateSessionButtonIsEnabled() const
{
	return SessionManager->GetSelectedSession().IsValid();
}


#undef LOCTEXT_NAMESPACE
