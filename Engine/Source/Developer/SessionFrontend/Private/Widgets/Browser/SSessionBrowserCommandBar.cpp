// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Browser/SSessionBrowserCommandBar.h"
#include "Widgets/SBoxPanel.h"


#define LOCTEXT_NAMESPACE "SSessionBrowserCommandBar"


/* SSessionBrowserCommandBar interface
 *****************************************************************************/

void SSessionBrowserCommandBar::Construct( const FArguments& InArgs )
{
	ChildSlot
	[
		SNew(SHorizontalBox)
	];
}


#undef LOCTEXT_NAMESPACE
