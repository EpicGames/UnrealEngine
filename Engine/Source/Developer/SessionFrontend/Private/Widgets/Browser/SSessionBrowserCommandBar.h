// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

/**
 * Implements the session browser's command bar widget.
 */
class SSessionBrowserCommandBar
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSessionBrowserCommandBar) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	void Construct( const FArguments& InArgs );
};
