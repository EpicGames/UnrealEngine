// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "InputCoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Styling/SlateColor.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Models/SessionBrowserTreeItems.h"
#include "Widgets/Images/SImage.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "SSessionBrowserTreeRow"

/**
 * Implements a row widget for the session browser tree.
 */
class SSessionBrowserTreeGroupRow
	: public STableRow<TSharedPtr<FSessionBrowserGroupTreeItem>>
{
public:

	SLATE_BEGIN_ARGS(SSessionBrowserTreeGroupRow) { }
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ARGUMENT(TSharedPtr<STableViewBase>, OwnerTableView)
		SLATE_ARGUMENT(TSharedPtr<FSessionBrowserGroupTreeItem>, Item)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		HighlightText = InArgs._HighlightText;
		Item = InArgs._Item;

		ChildSlot
			.Padding(0.0f, 2.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
					.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
					.BorderImage(this, &SSessionBrowserTreeGroupRow::HandleBorderBackgroundImage)
					.Padding(1.5f,3.0f)
					.ToolTipText(Item->GetToolTipText())
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
									.ColorAndOpacity(this, &SSessionBrowserTreeGroupRow::HandleGroupNameColorAndOpacity)
									.Text(this, &SSessionBrowserTreeGroupRow::HandleGroupNameText)
									.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
									.ShadowOffset(FVector2D(1.0f, 1.0f))
							]

						+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SImage)
								.Image(this, &SSessionBrowserTreeGroupRow::HandlePullDownImage)
								.ColorAndOpacity(IsHovered() ? FStyleColors::ForegroundHover : FStyleColors::Foreground)
							]
					]
			];

		STableRow<TSharedPtr<FSessionBrowserGroupTreeItem>>::ConstructInternal(
			STableRow<TSharedPtr<FSessionBrowserGroupTreeItem>>::FArguments()
				.ShowSelection(false)
				.Style(FAppStyle::Get(), "TableView.Row"),
			InOwnerTableView
		);
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

protected:

	// SWidget overrides

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			ToggleExpansion();

			return FReply::Handled();
		}
		else
		{
			return FReply::Unhandled();
		}
	}

private:

	/** Callback for getting the background image of the row's border. */
	const FSlateBrush* HandleBorderBackgroundImage() const
	{
		if (IsHovered())
		{
			return FAppStyle::GetBrush("Brushes.Hover");
		}
		else
		{
			return FAppStyle::GetBrush("Brushes.Header");
		}
	}

	/** Callback for getting the group name text's color. */
	FSlateColor HandleGroupNameColorAndOpacity() const
	{
		return (Item->GetChildren().Num() == 0)
			? FSlateColor::UseSubduedForeground()
			: FSlateColor::UseForeground();
	}

	/** Callback for getting the group name text. */
	FText HandleGroupNameText() const
	{
		int32 NumSessions = Item->GetChildren().Num();

		if ((NumSessions > 0) && (Item->GetChildren()[0]->GetType() == ESessionBrowserTreeNodeType::Session))
		{
			return FText::Format(LOCTEXT("GroupNameFormat", "{0} ({1})"), Item->GetGroupName(), FText::AsNumber(NumSessions));
		}

		return Item->GetGroupName();
	}

	/** Gets the image for the pull-down icon. */
	const FSlateBrush* HandlePullDownImage() const
	{
		return IsItemExpanded()
			? FAppStyle::GetBrush("Icons.ChevronUp")
			: FAppStyle::GetBrush("Icons.ChevronDown");
	}

private:

	/** The highlight string for this row. */
	TAttribute<FText> HighlightText;

	/** A reference to the tree item that is displayed in this row. */
	TSharedPtr<FSessionBrowserGroupTreeItem> Item;
};


#undef LOCTEXT_NAMESPACE
