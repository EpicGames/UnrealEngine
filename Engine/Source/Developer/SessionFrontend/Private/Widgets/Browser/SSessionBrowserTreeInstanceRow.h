// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Widgets/Views/STableViewBase.h"
#include "SessionFrontendStyle.h"
#include "Styling/StyleDefaults.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Models/SessionBrowserTreeItems.h"
#include "Widgets/Images/SImage.h"
#include "PlatformInfo.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "SSessionBrowserTreeRow"

/**
 * Implements a row widget for the session browser tree.
 */
class SSessionBrowserTreeInstanceRow
	: public SMultiColumnTableRow<TSharedPtr<FSessionBrowserTreeItem>>
{
public:

	SLATE_BEGIN_ARGS(SSessionBrowserTreeInstanceRow) { }
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ARGUMENT(TSharedPtr<STableViewBase>, OwnerTableView)
		SLATE_ARGUMENT(TSharedPtr<FSessionBrowserInstanceTreeItem>, Item)
		SLATE_ARGUMENT(bool, ShowSelection)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		HighlightText = InArgs._HighlightText;
		Item = InArgs._Item;

		SMultiColumnTableRow<TSharedPtr<FSessionBrowserTreeItem>>::Construct(FSuperRowType::FArguments().ShowSelection(InArgs._ShowSelection).Style(FAppStyle::Get(), "TableView.Row"), InOwnerTableView);
	}

public:

	/**
	 * Generates the widget for the specified column.
	 *
	 * @param ColumnName The name of the column to generate the widget for.
	 * @return The widget.
	 */
	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		if (ColumnName == "Device")
		{
			TSharedPtr<ISessionInstanceInfo> InstanceInfo = Item->GetInstanceInfo();

			if (InstanceInfo.IsValid())
			{
				const PlatformInfo::FTargetPlatformInfo* const PlatformInfo = PlatformInfo::FindPlatformInfo(*InstanceInfo->GetPlatformName());

				return SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.WidthOverride(24)
					.HeightOverride(24)
					[
						SNew(SImage)
						.Image((PlatformInfo) ? FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(EPlatformIconSize::Normal)) : FStyleDefaults::GetNoBrush())
					]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(2.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.ColorAndOpacity(this, &SSessionBrowserTreeInstanceRow::HandleTextColorAndOpacity)
					.Text(this, &SSessionBrowserTreeInstanceRow::HandleDeviceColumnText)
				];
			}
		}
		else if (ColumnName == "Level")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0, 0.0))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(this, &SSessionBrowserTreeInstanceRow::HandleTextColorAndOpacity)
						.Text(this, &SSessionBrowserTreeInstanceRow::HandleLevelColumnText)
				];
		}
		else if (ColumnName == "Name")
		{
			TSharedPtr<ISessionInstanceInfo> InstanceInfo = Item->GetInstanceInfo();

			if (InstanceInfo.IsValid())
			{
				return SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FSessionFrontendStyle::GetBrush("SessionBrowser.Row.Name"))
					.ColorAndOpacity(this, &SSessionBrowserTreeInstanceRow::GetNameImageColor)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("BoldFont"))
					.Text(FText::FromString(InstanceInfo->GetInstanceName()))
					.ColorAndOpacity(this, &SSessionBrowserTreeInstanceRow::HandleTextColorAndOpacity)
				];
			}
		}
		else if (ColumnName == "Status")
		{
			return SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(this, &SSessionBrowserTreeInstanceRow::HandleAuthorizedImage)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4.0f, 0.0f, 1.5f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.FilledCircle"))
				.ColorAndOpacity(this, &SSessionBrowserTreeInstanceRow::HandleStatusImageColor)
			];
		}
		else if (ColumnName == "Type")
		{
			TSharedPtr<ISessionInstanceInfo> InstanceInfo = Item->GetInstanceInfo();

			if (InstanceInfo.IsValid())
			{
				return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.ColorAndOpacity(this, &SSessionBrowserTreeInstanceRow::HandleTextColorAndOpacity)
					.Text(this, &SSessionBrowserTreeInstanceRow::HandleInstanceTypeText)
				];
			}
		}

		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:

	/** Callback for getting the image of the Authorized icon. */
	const FSlateBrush* HandleAuthorizedImage() const
	{
		TSharedPtr<ISessionInstanceInfo> InstanceInfo = Item->GetInstanceInfo();

		if (InstanceInfo.IsValid() && !InstanceInfo->IsAuthorized())
		{
			return FAppStyle::GetBrush("Icons.Lock");
		}

		return nullptr;
	}

	/** Callback for getting the text in the 'Device' column. */
	FText HandleDeviceColumnText() const
	{
		TSharedPtr<ISessionInstanceInfo> InstanceInfo = Item->GetInstanceInfo();

		if (InstanceInfo.IsValid())
		{
			return FText::FromString(InstanceInfo->GetDeviceName());
		}

		return FText::GetEmpty();
	}

	/** Callback for getting the image color for the name column. */
	FSlateColor GetNameImageColor() const
	{
		TSharedPtr<ISessionInstanceInfo> InstanceInfo = Item->GetInstanceInfo();

		if (InstanceInfo.IsValid())
		{
			return FLinearColor((GetTypeHash(InstanceInfo->GetInstanceId()) & 0xff) * 360.0f / 256.0f, 0.8f, 0.3f, 1.0f).HSVToLinearRGB();
		}

		return FLinearColor::Transparent;
	}

	/** Callback for getting the text color for the name column. */
	FSlateColor GetNameTextColor() const
	{
		FSlateColor Color = GetNameImageColor();

		if (Color == FLinearColor::Transparent)
		{
			Color = FSlateColor::UseSubduedForeground();;
		}

		return Color;
	}

	/** Callback for getting the border brush for this row. */
	const FSlateBrush* HandleInstanceBorderBrush() const
	{
		TSharedPtr<ISessionInstanceInfo> InstanceInfo = Item->GetInstanceInfo();

		if (InstanceInfo.IsValid())
		{
			if (FDateTime::UtcNow() - InstanceInfo->GetLastUpdateTime() < FTimespan::FromSeconds(10.0))
			{
				return FAppStyle::GetBrush("ErrorReporting.Box");
			}

			return FAppStyle::GetBrush("ErrorReporting.EmptyBox");
		}

		return nullptr;
	}

	/** Callback for getting the type of the session instance. */
	FText HandleInstanceTypeText() const
	{
		TSharedPtr<ISessionInstanceInfo> InstanceInfo = Item->GetInstanceInfo();

		if (InstanceInfo.IsValid())
		{
			return FText::FromString(InstanceInfo->GetInstanceType());
		}

		return FText::GetEmpty();
	}

	/** Callback for getting the instance's current level. */
	FText HandleLevelColumnText() const
	{
		TSharedPtr<ISessionInstanceInfo> InstanceInfo = Item->GetInstanceInfo();

		if (InstanceInfo.IsValid())
		{
			return FText::FromString(InstanceInfo->GetCurrentLevel());
		}

		return FText::GetEmpty();
	}

	/** Callback for getting the image of the Status icon. */
	FSlateColor HandleStatusImageColor() const
	{
		TSharedPtr<ISessionInstanceInfo> InstanceInfo = Item->GetInstanceInfo();

		if (InstanceInfo.IsValid())
		{
			if (FDateTime::UtcNow() - InstanceInfo->GetLastUpdateTime() < FTimespan::FromSeconds(10.0))
			{
				return FStyleColors::AccentGreen;
			}

			return FStyleColors::AccentGray;
		}

		return FStyleColors::Error;
	}

	/** Callback for getting the foreground text color. */
	FSlateColor HandleTextColorAndOpacity() const
	{
		TSharedPtr<ISessionInstanceInfo> InstanceInfo = Item->GetInstanceInfo();

		if (InstanceInfo.IsValid())
		{
			if (FDateTime::UtcNow() - InstanceInfo->GetLastUpdateTime() < FTimespan::FromSeconds(10.0))
			{
				return FSlateColor::UseForeground();
			}
		}

		return FSlateColor::UseSubduedForeground();
	}

private:

	/** The highlight string for this row. */
	TAttribute<FText> HighlightText;

	/** A reference to the tree item that is displayed in this row. */
	TSharedPtr<FSessionBrowserInstanceTreeItem> Item;
};


#undef LOCTEXT_NAMESPACE
