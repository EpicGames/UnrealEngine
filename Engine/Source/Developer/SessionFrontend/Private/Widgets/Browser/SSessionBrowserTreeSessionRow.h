// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Misc/Attribute.h"
#include "InputCoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Misc/App.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Models/SessionBrowserTreeItems.h"

#define LOCTEXT_NAMESPACE "SSessionBrowserTreeRow"

/**
 * Implements a row widget for the session browser tree.
 */
class SSessionBrowserTreeSessionRow
	: public STableRow<TSharedPtr<FSessionBrowserSessionTreeItem>>
{
public:

	SLATE_BEGIN_ARGS(SSessionBrowserTreeSessionRow) { }
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ARGUMENT(TSharedPtr<STableViewBase>, OwnerTableView)
		SLATE_ARGUMENT(TSharedPtr<FSessionBrowserSessionTreeItem>, Item)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		HighlightText = InArgs._HighlightText;
		Item = InArgs._Item;

		ChildSlot
			.Padding(0.0f, 2.0f, 0.0f, 0.0f)
			[
				SNew(SBox)
					.Padding(FMargin(0.0f, 3.0f, 16.0f, 3.0f))
					.ToolTipText(this, &SSessionBrowserTreeSessionRow::HandleBorderToolTipText)
					[
						SNew(SHorizontalBox)
						
						+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(2.0f, 2.0f, 2.0f, 2.0f)
							[
								SNew(SExpanderArrow, SharedThis(this))
									.IndentAmount(0.0f)
							]

						+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
									.Text(this, &SSessionBrowserTreeSessionRow::HandleSessionNameText)
									.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
							]
					]
			];

		STableRow<TSharedPtr<FSessionBrowserSessionTreeItem>>::ConstructInternal(
			STableRow<TSharedPtr<FSessionBrowserSessionTreeItem>>::FArguments()
				.ShowSelection(false)
				.Style(FAppStyle::Get(), "TableView.Row"),
			InOwnerTableView
		);
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

protected:

	FText SanitizeSessionName(const TSharedPtr<ISessionInfo>& SessionInfo) const
	{
		const FString& SessionName = SessionInfo->GetSessionName();
		const FString& SessionOwner = SessionInfo->GetSessionOwner();

		// return name of a launched session
		if (!SessionName.IsEmpty())
		{
			if (SessionInfo->GetSessionOwner() != FPlatformProcess::UserName(false))
			{
				return FText::Format(LOCTEXT("SessionNameFormat", "{0} - {1}"), FText::FromString(SessionName), FText::FromString(SessionOwner));
			}

			return FText::FromString(SessionName);
		}

		// generate name for a standalone session
		TArray<TSharedPtr<ISessionInstanceInfo>> Instances;
		SessionInfo->GetInstances(Instances);

		if (Instances.Num() > 0)
		{
			const TSharedPtr<ISessionInstanceInfo>& FirstInstance = Instances[0];

			if ((Instances.Num() == 1) && FApp::IsThisInstance(FirstInstance->GetInstanceId()))
			{
				return LOCTEXT("ThisApplicationSessionText", "This Application");
			}
		}

		if (SessionInfo->GetSessionOwner() != FPlatformProcess::UserName(false))
		{
			return FText::Format(LOCTEXT("UnnamedSessionFormat", "Unnamed - {0}"), FText::FromString(SessionOwner));
		}

		return LOCTEXT("UnnamedSessionText", "Unnamed");
	}

protected:

	// SWidget overrides

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			ToggleExpansion();

			return FReply::Handled();
		}
		else
		{
			return FReply::Unhandled();
		}
	}

private:

	/** Callback for getting the background image of the row's border. */
	const FSlateBrush* HandleBorderBackgroundImage() const
	{
		if (IsHovered())
		{
			return FAppStyle::GetBrush("Brushes.Hover");
		}
		else
		{
			return FAppStyle::GetBrush("Brushes.Header");
		}
	}

	/** Callback for getting the text of the row border's tool tip. */
	FText HandleBorderToolTipText() const
	{
		FTextBuilder ToolTipTextBuilder;

		TSharedPtr<ISessionInfo> SessionInfo = Item->GetSessionInfo();

		if (SessionInfo.IsValid())
		{
			const FCoreTexts& CoreTexts = FCoreTexts::Get();

			ToolTipTextBuilder.AppendLineFormat(LOCTEXT("SessionToolTipSessionId", "Session ID: {0}"), FText::FromString(SessionInfo->GetSessionId().ToString(EGuidFormats::DigitsWithHyphensInBraces)));
			ToolTipTextBuilder.AppendLine();
			ToolTipTextBuilder.AppendLineFormat(LOCTEXT("SessionToolTipNumInstances", "Total Instances: {0}"), FText::AsNumber(SessionInfo->GetNumInstances()));
			ToolTipTextBuilder.AppendLineFormat(LOCTEXT("SessionToolTipIsStandalone", "Is Standalone: {0}"), SessionInfo->IsStandalone() ? CoreTexts.Yes : CoreTexts.No);
			ToolTipTextBuilder.AppendLine();
			ToolTipTextBuilder.AppendLineFormat(LOCTEXT("SessionToolTipLastUpdateTime", "Last Update Time: {0}"), FText::AsDateTime(SessionInfo->GetLastUpdateTime()));
		}

		return ToolTipTextBuilder.ToText();
	}

	/** Callback for getting the name of the session. */
	FText HandleSessionNameText() const
	{
		return SanitizeSessionName(Item->GetSessionInfo());
	}

private:

	/** The highlight string for this row. */
	TAttribute<FText> HighlightText;

	/** A reference to the tree item that is displayed in this row. */
	TSharedPtr<FSessionBrowserSessionTreeItem> Item;
};


#undef LOCTEXT_NAMESPACE
