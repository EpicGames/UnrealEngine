// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Console/SSessionConsole.h"
#include "DesktopPlatformModule.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/SOverlay.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "Models/SessionConsoleCommands.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Console/SSessionConsoleLogTableRow.h"
#include "Widgets/Console/SSessionConsoleCommandBar.h"
#include "Widgets/Console/SSessionConsoleFilterBar.h"
#include "Widgets/Console/SSessionConsoleShortcutWindow.h"
#include "Widgets/Console/SSessionConsoleToolbar.h"
#include "Widgets/Layout/SExpandableArea.h"


#define LOCTEXT_NAMESPACE "SSessionConsolePanel"


/* SSessionConsolePanel structors
 *****************************************************************************/

SSessionConsole::~SSessionConsole()
{
	if (SessionManager.IsValid())
	{
		SessionManager->OnInstanceSelectionChanged().RemoveAll(this);
		SessionManager->OnLogReceived().RemoveAll(this);
		SessionManager->OnSelectedSessionChanged().RemoveAll(this);
	}
}


/* SSessionConsolePanel interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SSessionConsole::Construct(const FArguments& InArgs, TSharedRef<ISessionManager> InSessionManager)
{
	SessionManager = InSessionManager;
	ShouldScrollToLast = true;

	// create and bind the commands
	UICommandList = MakeShareable(new FUICommandList);
	BindCommands();

	ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
			[
				SNew(SVerticalBox)
					.IsEnabled(this, &SSessionConsole::HandleMainContentIsEnabled)

				+ SVerticalBox::Slot()
					.AutoHeight()
					[
						// toolbar
						SNew(SSessionConsoleToolbar, UICommandList.ToSharedRef())
					]

				+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 4.0f, 0.0f, 0.0f)
					[
						// filter bar
						SNew(SExpandableArea)
							.AreaTitle(LOCTEXT("FilterBarAreaTitle", "Log Filter"))
							.InitiallyCollapsed(true)
							.Padding(FMargin(8.0f, 6.0f))
							.BodyContent()
							[
								SAssignNew(FilterBar, SSessionConsoleFilterBar)
									.OnFilterChanged(this, &SSessionConsole::HandleFilterChanged)
							]
					]

				//content area for the log
				+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					.Padding(0.0f, 4.0f, 0.0f, 0.0f)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							[
								// log list
								SNew(SBorder)
									.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
									.Padding(0.0f)
									[
										SAssignNew(LogListView, SListView<TSharedPtr<FSessionLogMessage>>)
											.ListItemsSource(&LogMessages)
											.SelectionMode(ESelectionMode::Multi)
											.OnGenerateRow(this, &SSessionConsole::HandleLogListGenerateRow)
											.OnItemScrolledIntoView(this, &SSessionConsole::HandleLogListItemScrolledIntoView)
											.HeaderRow
											(
												SNew(SHeaderRow)

												+ SHeaderRow::Column("Verbosity")
													.DefaultLabel(LOCTEXT("LogListVerbosityColumnHeader", " "))
													.FixedWidth(24.0f)

												+ SHeaderRow::Column("Instance")
													.DefaultLabel(LOCTEXT("LogListHostNameColumnHeader", "Instance"))
													.FillWidth(0.20f)

												+ SHeaderRow::Column("TimeSeconds")
													.DefaultLabel(LOCTEXT("LogListTimestampColumnHeader", "Seconds"))
													.FillWidth(0.10f)

												+ SHeaderRow::Column("Message")
													.DefaultLabel(LOCTEXT("LogListTextColumnHeader", "Message"))
													.FillWidth(0.70f)
											)
									]
						]

						//Shortcut buttons
						+ SHorizontalBox::Slot()
							.FillWidth(0.2f)
							[
								SAssignNew(ShortcutWindow, SSessionConsoleShortcutWindow)
									.OnCommandSubmitted(this, &SSessionConsole::HandleCommandSubmitted)
							]
					]

				+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 4.0f, 0.0f, 0.0f)
					[
						SNew(SBorder)
							.Padding(FMargin(8.0f, 6.0f))
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								// command bar
								SAssignNew(CommandBar, SSessionConsoleCommandBar)
									.OnCommandSubmitted(this, &SSessionConsole::HandleCommandSubmitted)
									.OnPromoteToShortcutClicked(this, &SSessionConsole::HandleCommandBarPromoteToShortcutClicked)
							]
					]
			]

		+ SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NotificationList.ItemBackground"))
					.Padding(8.0f)
					.Visibility(this, &SSessionConsole::HandleSelectSessionOverlayVisibility)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("SelectSessionOverlayText", "Please select at least one instance from the Session Browser"))
					]
			]
	];

	SessionManager->OnInstanceSelectionChanged().AddSP(this, &SSessionConsole::HandleSessionManagerInstanceSelectionChanged);
	SessionManager->OnLogReceived().AddSP(this, &SSessionConsole::HandleSessionManagerLogReceived);
	SessionManager->OnSelectedSessionChanged().AddSP(this, &SSessionConsole::HandleSessionManagerSelectedSessionChanged);

	ReloadLog(true);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SSessionConsolePanel implementation
 *****************************************************************************/

void SSessionConsole::BindCommands()
{
	FSessionConsoleCommands::Register();

	const FSessionConsoleCommands& Commands = FSessionConsoleCommands::Get();

	UICommandList->MapAction(
		Commands.Clear,
		FExecuteAction::CreateSP(this, &SSessionConsole::HandleClearActionExecute),
		FCanExecuteAction::CreateSP(this, &SSessionConsole::HandleClearActionCanExecute));

	UICommandList->MapAction(
		Commands.SessionCopy,
		FExecuteAction::CreateSP(this, &SSessionConsole::HandleCopyActionExecute),
		FCanExecuteAction::CreateSP(this, &SSessionConsole::HandleCopyActionCanExecute));

	UICommandList->MapAction(
		Commands.SessionSave,
		FExecuteAction::CreateSP(this, &SSessionConsole::HandleSaveActionExecute),
		FCanExecuteAction::CreateSP(this, &SSessionConsole::HandleSaveActionCanExecute));
}


void SSessionConsole::ClearLog()
{
	LogMessages.Reset();
	LogListView->RequestListRefresh();
}


void SSessionConsole::CopyLog()
{
	TArray<TSharedPtr<FSessionLogMessage>> SelectedItems = LogListView->GetSelectedItems();

	if (SelectedItems.Num() == 0)
	{
		return;
	}

	FString SelectedText;

	for (const auto& Item : SelectedItems)
	{
		SelectedText += FString::Printf(TEXT("%s [%s] %09.3f: %s"), *Item->Time.ToString(), *Item->InstanceName, Item->TimeSeconds, *Item->Text);
		SelectedText += LINE_TERMINATOR;
	}

	FPlatformApplicationMisc::ClipboardCopy(*SelectedText);
}


void SSessionConsole::ReloadLog(bool FullyReload)
{
	// reload log list
	if (FullyReload)
	{
		AvailableLogs.Reset();

		const auto& SelectedInstances = SessionManager->GetSelectedInstances();

		for (const auto& Instance : SelectedInstances)
		{
			const TArray<TSharedPtr<FSessionLogMessage>>& InstanceLog = Instance->GetLog();

			for (const auto& LogMessage : InstanceLog)
			{
				AvailableLogs.HeapPush(LogMessage, FSessionLogMessage::TimeComparer());
			}
		}

		CommandBar->SetNumSelectedInstances(SelectedInstances.Num());
	}

	LogMessages.Reset();

	// filter log list
	FilterBar->ResetFilter();

	for (const auto& LogMessage : AvailableLogs)
	{
		if (FilterBar->FilterLogMessage(LogMessage.ToSharedRef()))
		{
			LogMessages.Add(LogMessage);
		}
	}

	// refresh list view
	LogListView->RequestListRefresh();

	if (LogMessages.Num() > 0)
	{
		LogListView->RequestScrollIntoView(LogMessages.Last());
	}
}


void SSessionConsole::SaveLog()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

	if (DesktopPlatform == nullptr)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("SaveLogDialogUnsupportedError", "Saving is not supported on this platform!"));

		return;
	}

	TArray<FString> Filenames;

	// open file dialog
	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

	if (!DesktopPlatform->SaveFileDialog(
		ParentWindowHandle,
		LOCTEXT("SaveLogDialogTitle", "Save Log As...").ToString(),
		LastLogFileSaveDirectory,
		TEXT("Session.log"),
		TEXT("Log Files (*.log)|*.log"),
		EFileDialogFlags::None,
		Filenames))
	{
		return;
	}

	// no log file selected?
	if (Filenames.Num() == 0)
	{
		return;
	}

	FString Filename = Filenames[0];

	// keep path as default for next time
	LastLogFileSaveDirectory = FPaths::GetPath(Filename);

	// add a file extension if none was provided
	if (FPaths::GetExtension(Filename).IsEmpty())
	{
		Filename += Filename + TEXT(".log");
	}

	// save file
	FArchive* LogFile = IFileManager::Get().CreateFileWriter(*Filename);

	if (LogFile != nullptr)
	{
		for (const auto& LogMessage : LogMessages)
		{
			FString LogEntry = FString::Printf(TEXT("%s [%s] %09.3f: %s"),
				*LogMessage->Time.ToString(),
				*LogMessage->InstanceName,
				LogMessage->TimeSeconds,
				*LogMessage->Text) + LINE_TERMINATOR;

			LogFile->Serialize(TCHAR_TO_ANSI(*LogEntry), LogEntry.Len());
		}

		LogFile->Close();
		delete LogFile;
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("SaveLogDialogFileError", "Failed to open the specified file for saving!"));
	}
}


void SSessionConsole::SendCommand(const FString& CommandString)
{
	if (CommandString.IsEmpty())
	{
		return;
	}

	for (auto& Instance : SessionManager->GetSelectedInstances())
	{
		Instance->ExecuteCommand(CommandString);
	}
}



/* SWidget implementation
 *****************************************************************************/

FReply SSessionConsole::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.IsControlDown())
	{
		if (InKeyEvent.GetKey() == EKeys::C)
		{
			CopyLog();

			return FReply::Handled();
		}
	
		if (InKeyEvent.GetKey() == EKeys::S)
		{
			SaveLog();

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}


/* SSessionConsolePanel event handlers
 *****************************************************************************/

void SSessionConsole::HandleClearActionExecute()
{
	ClearLog();
}


bool SSessionConsole::HandleClearActionCanExecute()
{
	return (LogMessages.Num() > 0);
}


void SSessionConsole::HandleCommandBarPromoteToShortcutClicked(const FString& CommandString)
{
	ShortcutWindow->AddShortcut(CommandString, CommandString);
}


void SSessionConsole::HandleCommandSubmitted(const FString& CommandString)
{
	SendCommand(CommandString);
}


void SSessionConsole::HandleCopyActionExecute()
{
	CopyLog();
}


bool SSessionConsole::HandleCopyActionCanExecute()
{
	return (LogListView->GetNumItemsSelected() > 0);
}


void SSessionConsole::HandleFilterChanged()
{
	HighlightText = FilterBar->GetFilterText().ToString();

	ReloadLog(false);
}


void SSessionConsole::HandleLogListItemScrolledIntoView(TSharedPtr<FSessionLogMessage> Item, const TSharedPtr<ITableRow>& TableRow)
{
	if (LogMessages.Num() > 0)
	{
		ShouldScrollToLast = LogListView->IsItemVisible(LogMessages.Last());
	}
	else
	{
		ShouldScrollToLast = true;
	}
}


TSharedRef<ITableRow> SSessionConsole::HandleLogListGenerateRow(TSharedPtr<FSessionLogMessage> Message, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SSessionConsoleLogTableRow, OwnerTable)
		.HighlightText(this, &SSessionConsole::HandleLogListGetHighlightText)
		.LogMessage(Message)
		.ToolTipText(FText::FromString(Message->Text));
}


FText SSessionConsole::HandleLogListGetHighlightText() const
{
	return FText::FromString(HighlightText); //FilterBar->GetFilterText();
}


bool SSessionConsole::HandleMainContentIsEnabled() const
{
	return (SessionManager->GetSelectedInstances().Num() > 0);
}


void SSessionConsole::HandleSaveActionExecute()
{
	SaveLog();
}


bool SSessionConsole::HandleSaveActionCanExecute()
{
	return (LogMessages.Num() > 0);
}


EVisibility SSessionConsole::HandleSelectSessionOverlayVisibility() const
{
	if (SessionManager->GetSelectedInstances().Num() > 0)
	{
		return EVisibility::Hidden;
	}

	return EVisibility::Visible;
}


void SSessionConsole::HandleSessionManagerInstanceSelectionChanged(const TSharedPtr<ISessionInstanceInfo>& /*Instance*/, bool /*Selected*/)
{
	ReloadLog(true);
}


void SSessionConsole::HandleSessionManagerLogReceived(const TSharedRef<ISessionInfo>& Session, const TSharedRef<ISessionInstanceInfo>& Instance, const TSharedRef<FSessionLogMessage>& Message)
{
	if (!SessionManager->IsInstanceSelected(Instance))
	{
		return;
	}

	// Adds the log to the available messages even if it is filtered out, so that it cannot be lost
	AvailableLogs.Add(Message);
	if (!FilterBar->FilterLogMessage(Message))
	{
		return;
	}

	LogMessages.Add(Message);

	LogListView->RequestListRefresh();

	if (ShouldScrollToLast)
	{
		LogListView->RequestScrollIntoView(Message);
	}
}


void SSessionConsole::HandleSessionManagerSelectedSessionChanged(const TSharedPtr<ISessionInfo>& SelectedSession)
{
	ReloadLog(true);
}


#undef LOCTEXT_NAMESPACE
