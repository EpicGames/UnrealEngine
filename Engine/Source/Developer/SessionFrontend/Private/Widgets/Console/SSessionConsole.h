// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "ISessionInstanceInfo.h"
#include "ISessionInfo.h"
#include "SessionLogMessage.h"
#include "ISessionManager.h"

class FUICommandList;
class SSessionConsoleCommandBar;
class SSessionConsoleFilterBar;
class SSessionConsoleShortcutWindow;

/**
 * Implements the session console panel.
 *
 * This panel receives console log messages from a remote engine session and can also send
 * console commands to it.
 */
class SSessionConsole
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSessionConsole) { }
	SLATE_END_ARGS()

public:

	/** Destructor. */
	~SSessionConsole();

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InSessionManager The session manager to use.
	 */
	void Construct(const FArguments& InArgs, TSharedRef<ISessionManager> InSessionManager);

protected:

	/** Binds the device commands on our toolbar. */
	void BindCommands();

	/**
	 * Clears the log list view.
	 *
	 * @see CopyLog, ReloadLog, SaveLog
	 */
	void ClearLog();

	/**
	 * Copies the selected log messages to the clipboard.
	 *
	 * @see ClearLog, ReloadLog, SaveLog
	 */
	void CopyLog();

	/**
	 * Reloads the log messages for the currently selected engine instances.
	 *
	 * @param FullyReload Whether to fully reload the log entries or only re-apply filtering.
	 * @see ClearLog, CopyLog, SaveLog
	 */
	void ReloadLog(bool FullyReload);

	/**
	 * Saves all log messages to a file.
	 *
	 * @see ClearLog, CopyLog, ReloadLog
	 */
	void SaveLog();

	/**
	 * Sends the command entered into the input field.
	 *
	 * @param CommandString The command string to send.
	 */
	void SendCommand(const FString& CommandString);

protected:

	// SCompoundWidget overrides

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:

	/** Callback for executing the 'Clear' action. */
	void HandleClearActionExecute();

	/** Callback for determining the 'Clear' action can execute. */
	bool HandleClearActionCanExecute();

	/** Callback for executing the 'Copy' action. */
	void HandleCopyActionExecute();

	/** Callback for determining the 'Copy' action can execute. */
	bool HandleCopyActionCanExecute();

	/** Callback for executing the 'Save' action. */
	void HandleSaveActionExecute();

	/** Callback for determining the 'Save' action can execute. */
	bool HandleSaveActionCanExecute();

	/** Callback for promoting console command to shortcuts. */
	void HandleCommandBarPromoteToShortcutClicked(const FString& CommandString);

	/** Callback for submitting console commands. */
	void HandleCommandSubmitted(const FString& CommandString);

	/** Callback for changing the filter settings. */
	void HandleFilterChanged();

	/** Callback for scrolling a log item into view. */
	void HandleLogListItemScrolledIntoView(TSharedPtr<FSessionLogMessage> Item, const TSharedPtr<ITableRow>& TableRow);

	/** Callback for generating a row widget for the log list view. */
	TSharedRef<ITableRow> HandleLogListGenerateRow(TSharedPtr<FSessionLogMessage> Message, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback for getting the highlight string for log messages. */
	FText HandleLogListGetHighlightText() const;

	/** Callback for selecting log messages. */
	void HandleLogListSelectionChanged(TSharedPtr<FSessionLogMessage> InItem, ESelectInfo::Type SelectInfo);

	/** Callback for getting the enabled state of the console box. */
	bool HandleMainContentIsEnabled() const;

	/** Callback for determining the visibility of the 'Select a session' overlay. */
	EVisibility HandleSelectSessionOverlayVisibility() const;

	/** Callback for changing the engine instance selection. */
	void HandleSessionManagerInstanceSelectionChanged(const TSharedPtr<ISessionInstanceInfo>& Instance, bool Selected);

	/** Callback for received log entries. */
	void HandleSessionManagerLogReceived(const TSharedRef<ISessionInfo>& Session, const TSharedRef<ISessionInstanceInfo>& Instance, const TSharedRef<FSessionLogMessage>& Message);

	/** Callback for changing the selected session. */
	void HandleSessionManagerSelectedSessionChanged(const TSharedPtr<ISessionInfo>& SelectedSession);

private:

	/** Holds an unfiltered list of available log messages. */
	TArray<TSharedPtr<FSessionLogMessage>> AvailableLogs;

	/** Holds the command bar. */
	TSharedPtr<SSessionConsoleCommandBar> CommandBar;

	/** Holds the filter bar. */
	TSharedPtr<SSessionConsoleFilterBar> FilterBar;

	/** Holds the find bar. */
	TSharedPtr<SSearchBox> FindBar;

	/** Holds the highlight text. */
	FString HighlightText;

	/** Holds the directory where the log file was last saved to. */
	FString LastLogFileSaveDirectory;

	/** Holds the log list view. */
 	TSharedPtr<SListView<TSharedPtr<FSessionLogMessage>>> LogListView;

 	/** Holds the filtered list of log messages. */
 	TArray<TSharedPtr<FSessionLogMessage>> LogMessages;

	/** Holds the session manager. */
	TSharedPtr<ISessionManager> SessionManager;

	/** Holds the shortcut window. */
	TSharedPtr<SSessionConsoleShortcutWindow> ShortcutWindow;

	/** Holds a flag indicating whether the log list should auto-scroll to the last item. */
	bool ShouldScrollToLast;

	/** The command list for controlling the device */
	TSharedPtr<FUICommandList> UICommandList;
};
