// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Console/SSessionConsoleCommandBar.h"
#include "Widgets/SBoxPanel.h"
#include "SlateOptMacros.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSuggestionTextBox.h"


#define LOCTEXT_NAMESPACE "SSessionConsoleCommandBar"


/* SSessionConsoleCommandBar interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SSessionConsoleCommandBar::Construct(const FArguments& InArgs)
{
	OnCommandSubmitted = InArgs._OnCommandSubmitted;
	OnPromoteToShortcutClicked = InArgs._OnPromoteToShortcutClicked;

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				// command input
				SAssignNew(InputTextBox, SSuggestionTextBox)
					.ClearKeyboardFocusOnCommit(false)
					.OnShowingHistory(this, &SSessionConsoleCommandBar::HandleInputTextShowingHistory)
					.OnShowingSuggestions(this, &SSessionConsoleCommandBar::HandleInputTextShowingSuggestions)
					.OnTextChanged(this, &SSessionConsoleCommandBar::HandleInputTextChanged)
					.OnTextCommitted(this, &SSessionConsoleCommandBar::HandleInputTextCommitted)
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				// send button
				SAssignNew(SendButton, SButton)
					.ContentPadding(FMargin(6.0f, 2.0f))
					.IsEnabled(false)
					.OnClicked(this, &SSessionConsoleCommandBar::HandleSendButtonClicked)
					.ToolTipText(LOCTEXT("SendButtonTooltip", "Send the command"))
					[
						SNew(STextBlock)
							.Text(LOCTEXT("SendButtonLabel", "Send Command"))
					]
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				// send button
				SAssignNew(PromoteToShortcutButton, SButton)
					.ContentPadding(FMargin(6.0f, 2.0f))
					.IsEnabled(false)
					.OnClicked(this, &SSessionConsoleCommandBar::HandlePromoteToShortcutButtonClicked)
					.ToolTipText(LOCTEXT("PromoteConsoleCommandButtonTooltip", "Promote Command to Shortcut"))
					[
						SNew(STextBlock)
							.Text(LOCTEXT("PromoteConsoleCommandButtonLabel", "Promote to Shortcut"))
					]
			]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void SSessionConsoleCommandBar::SetNumSelectedInstances(int Count)
{
	FString CommandString = InputTextBox->GetText().ToString();
	CommandString.TrimStartInline();

	bool bEnableButtons = (Count > 0) && !CommandString.IsEmpty();

	InputTextBox->SetEnabled(Count > 0);
	SendButton->SetEnabled(bEnableButtons);
	PromoteToShortcutButton->SetEnabled(bEnableButtons);
}


/* SSessionConsoleCommandBar implementation
 *****************************************************************************/

void SSessionConsoleCommandBar::SubmitCommand(const FString& Command)
{
	OnCommandSubmitted.ExecuteIfBound(Command);

	CommandHistory.Remove(Command);
	CommandHistory.Add(Command);

	InputTextBox->SetText(FText::GetEmpty());
}


/* SSessionConsoleCommandBar event handlers
 *****************************************************************************/

void SSessionConsoleCommandBar::HandleInputTextChanged(const FText& InText)
{
	FString CommandString = InputTextBox->GetText().ToString();
	CommandString.TrimStartInline();

	SendButton->SetEnabled(!CommandString.IsEmpty());
	PromoteToShortcutButton->SetEnabled(!CommandString.IsEmpty());	
}


void SSessionConsoleCommandBar::HandleInputTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		SubmitCommand(InText.ToString());
	}
}


void SSessionConsoleCommandBar::HandleInputTextShowingHistory(TArray<FString>& OutHistory)
{
	OutHistory = CommandHistory;
}


void SSessionConsoleCommandBar::HandleInputTextShowingSuggestions(const FString& Text, TArray<FString>& OutSuggestions)
{
	// @todo gmp: implement remote auto-complete
}


FReply SSessionConsoleCommandBar::HandlePromoteToShortcutButtonClicked()
{
	if (OnPromoteToShortcutClicked.IsBound())
	{
		FString CommandString = InputTextBox->GetText().ToString();
		CommandString.TrimStartInline();
		OnPromoteToShortcutClicked.Execute(CommandString);
	}

	return FReply::Handled();
}


FReply SSessionConsoleCommandBar::HandleSendButtonClicked()
{
	SubmitCommand(InputTextBox->GetText().ToString());

	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
