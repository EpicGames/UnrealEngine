// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SButton;

/** Delegate type for submitting console commands. */
DECLARE_DELEGATE_OneParam(FOnSessionConsoleCommandSubmitted, const FString& /*Command*/)


/**
 * Implements the session console's command bar widget.
 */
class SSessionConsoleCommandBar
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSessionConsoleCommandBar) { }

		/** Called when the filter settings have changed. */
		SLATE_EVENT(FOnSessionConsoleCommandSubmitted, OnCommandSubmitted)

		/** Called when the promote to shortcut button is clicked. */
		SLATE_EVENT(FOnSessionConsoleCommandSubmitted, OnPromoteToShortcutClicked)

	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	void Construct(const FArguments& InArgs);

	/**
	 * Sets the number of selected engine instances.
	 *
	 * @param Count Number of selected instances.
	 */
	void SetNumSelectedInstances(int Count);

protected:

	/**
	 * Submits the entered command.
	 *
	 * @param Command the command to submit.
	 */
	void SubmitCommand(const FString& Command);

private:

	/** Handles changing the input text box's content. */
	void HandleInputTextChanged(const FText& InText);

	/** Handles committing the input text box's content. */
	void HandleInputTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

	/** Handles showing a history in the input text box. */
	void HandleInputTextShowingHistory(TArray<FString>& OutHistory);

	/** Handles showing suggestions in the input text box. */
	void HandleInputTextShowingSuggestions(const FString& Text, TArray<FString>& OutSuggestions);

	/** Handles clicking the promote to shortcut button. */
	FReply HandlePromoteToShortcutButtonClicked();

	/** Handles clicking the send button. */
	FReply HandleSendButtonClicked();

private:

	/** Holds the command history. */
	TArray<FString> CommandHistory;

	/** Holds the input text box. */
	TSharedPtr<SSuggestionTextBox> InputTextBox;

	/** Holds the send button. */
	TSharedPtr<SButton> SendButton;

	/** Holds the promote to shortcut button. */
	TSharedPtr<SButton> PromoteToShortcutButton;

private:

	/** Holds a delegate that is executed when a command is submitted. */
	FOnSessionConsoleCommandSubmitted OnCommandSubmitted;

	/** Holds a delegate that is executed when the Promote To Shortcut button is clicked. */
	FOnSessionConsoleCommandSubmitted OnPromoteToShortcutClicked;
};
