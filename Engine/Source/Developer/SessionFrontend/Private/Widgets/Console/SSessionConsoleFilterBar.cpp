// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Console/SSessionConsoleFilterBar.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "SSessionConsoleFilterBar"


/* SSessionConsoleFilterBar interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SSessionConsoleFilterBar::Construct(const FArguments& InArgs)
{
	OnFilterChanged = InArgs._OnFilterChanged;

	// initialize verbosity filters
	AddVerbosityFilter(ELogVerbosity::Fatal, LOCTEXT("FatalVerbosityFilterTooltip", "Fatal errors").ToString(), "Icons.Error");
	AddVerbosityFilter(ELogVerbosity::Error, LOCTEXT("ErrorVerbosityFilterTooltip", "Errors").ToString(), "Icons.Error");
	AddVerbosityFilter(ELogVerbosity::Warning, LOCTEXT("WarningVerbosityFilterTooltip", "Warnings").ToString(), "Icons.Warning");
	AddVerbosityFilter(ELogVerbosity::Log, LOCTEXT("LogVerbosityFilterTooltip", "Log Messages").ToString(), "Icons.Info");
	AddVerbosityFilter(ELogVerbosity::Display, LOCTEXT("DisplayVerbosityFilterTooltip", "Display Messages").ToString(), "Icons.Info");
	AddVerbosityFilter(ELogVerbosity::Verbose, LOCTEXT("VerboseVerbosityFilterTooltip", "Verbose Messages").ToString(), "Icons.Info");
	AddVerbosityFilter(ELogVerbosity::VeryVerbose, LOCTEXT("VeryVerboseVerbosityFilterTooltip", "Very Verbose Messages").ToString(), "Icons.Info");

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				// search box
				SAssignNew(FilterStringTextBox, SSearchBox)
					.HintText(LOCTEXT("SearchBoxHint", "Search log messages"))
					.OnTextChanged(this, &SSessionConsoleFilterBar::HandleFilterStringTextChanged)
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				// highlight only check box
				SAssignNew(HighlightOnlyCheckBox, SCheckBox)
					.Padding(FMargin(6.0, 2.0))
					.OnCheckStateChanged(this, &SSessionConsoleFilterBar::HandleHighlightOnlyCheckBoxCheckStateChanged)
					.ToolTipText(LOCTEXT("HighlightOnlyCheckBoxTooltip", "Only highlight the search text instead of filtering the list of log messages"))
					[
						SNew(STextBlock)
							.Text(LOCTEXT("HighlightOnlyCheckBoxLabel", "Highlight Only"))
					]
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.Padding(16.0f, 0.0f, 0.0f, 0.0f)
			[
				// category filter
				SNew(SComboButton)
					.ButtonContent()
					[
						SNew(STextBlock)
							.Text(LOCTEXT("CategoryComboButtonText", "Categories"))
					]
					.ContentPadding(FMargin(6.0f, 2.0f))
					.MenuContent()
					[
						SAssignNew(CategoriesListView, SListView<FSessionConsoleCategoryFilterPtr>)
							.ListItemsSource(&CategoriesList)
							.OnGenerateRow(this, &SSessionConsoleFilterBar::HandleCategoryFilterGenerateRow)
					]
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			[
				// verbosity filter
				SNew(SComboButton)
					.ButtonContent()
					[
						SNew(STextBlock)
							.Text(LOCTEXT("VerbosityComboButtonText", "Verbosities"))
					]
					.ContentPadding(FMargin(6.0f, 2.0f))
					.MenuContent()
					[
						SAssignNew(VerbositiesListView, SListView<FSessionConsoleVerbosityFilterPtr>)
							.ListItemsSource(&VerbositiesList)
							.OnGenerateRow(this, &SSessionConsoleFilterBar::HandleVerbosityFilterGenerateRow)
					]
			]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


bool SSessionConsoleFilterBar::FilterLogMessage(const TSharedRef<FSessionLogMessage>& LogMessage)
{
	// create or update category counter
	int32& CategoryCounter = CategoryCounters.FindOrAdd(LogMessage->Category);

	if (CategoryCounter == 0)
	{
		AddCategoryFilter(LogMessage->Category);
	}

	++CategoryCounter;

	// update the verbosity counter
	++VerbosityCounters.FindOrAdd(LogMessage->Verbosity);

	// filter the log message
	if (DisabledCategories.Contains(LogMessage->Category) ||
		DisabledVerbosities.Contains(LogMessage->Verbosity))
	{
		return false;
	}

	if (HighlightOnlyCheckBox->IsChecked() || FilterStringTextBox->GetText().IsEmpty())
	{
		return true;
	}

	return (LogMessage->Text.Contains(FilterStringTextBox->GetText().ToString()));
}


void SSessionConsoleFilterBar::ResetFilter()
{
	CategoriesList.Reset();
	CategoryCounters.Reset();
	VerbosityCounters.Reset();

	CategoriesListView->RequestListRefresh();
}


/* SSessionConsoleFilterBar implementation
 *****************************************************************************/

void SSessionConsoleFilterBar::AddCategoryFilter(const FName& Category)
{
	CategoriesList.Add(MakeShareable(
		new FSessionConsoleCategoryFilter(Category,
			!DisabledCategories.Contains(Category),
			FOnSessionConsoleCategoryFilterStateChanged::CreateSP(this, &SSessionConsoleFilterBar::HandleCategoryFilterStateChanged)
		)
	));

	CategoriesListView->RequestListRefresh();
}


void SSessionConsoleFilterBar::AddVerbosityFilter(ELogVerbosity::Type Verbosity, const FString& Name, const FName& Icon)
{
	VerbositiesList.Add(MakeShareable(
		new FSessionConsoleVerbosityFilter(Verbosity, FAppStyle::GetBrush(Icon), true, Name,
			FOnSessionConsoleVerbosityFilterStateChanged::CreateSP(this, &SSessionConsoleFilterBar::HandleVerbosityFilterStateChanged))
		)
	);
}


/* SSessionConsoleFilterBar callbacks
 *****************************************************************************/

TSharedRef<ITableRow> SSessionConsoleFilterBar::HandleCategoryFilterGenerateRow(FSessionConsoleCategoryFilterPtr Filter, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<FSessionConsoleCategoryFilterPtr>, OwnerTable)
		[
			SNew(SCheckBox)
				.IsChecked(Filter.Get(), &FSessionConsoleCategoryFilter::GetCheckStateFromIsEnabled)
				.Padding(FMargin(6.0, 2.0))
				.OnCheckStateChanged(Filter.Get(), &FSessionConsoleCategoryFilter::EnableFromCheckState)
				[
					SNew(STextBlock)
						.Text(this, &SSessionConsoleFilterBar::HandleCategoryFilterGetRowText, Filter)
				]
		];
}


FText SSessionConsoleFilterBar::HandleCategoryFilterGetRowText(FSessionConsoleCategoryFilterPtr Filter) const
{
	return FText::Format(LOCTEXT("CategoryFilterRowFmt", "{0} ({1})"), FText::FromName(Filter->GetCategory()), FText::AsNumber(CategoryCounters.FindRef(Filter->GetCategory())));
}


void SSessionConsoleFilterBar::HandleCategoryFilterStateChanged(const FName& Category, bool Enabled)
{
	if (Enabled)
	{
		DisabledCategories.Remove(Category);
	}
	else
	{
		DisabledCategories.AddUnique(Category);
	}

	OnFilterChanged.ExecuteIfBound();
}


void SSessionConsoleFilterBar::HandleFilterStringTextChanged(const FText& NewText)
{
	OnFilterChanged.ExecuteIfBound();
}


void SSessionConsoleFilterBar::HandleHighlightOnlyCheckBoxCheckStateChanged(ECheckBoxState CheckedState)
{
	OnFilterChanged.ExecuteIfBound();
}


FText SSessionConsoleFilterBar::HandleVerbosityFilterGetRowText(FSessionConsoleVerbosityFilterPtr Filter) const
{
	return FText::Format(LOCTEXT("VerbosityFilterRowFmt", "{0} ({1})"), FText::FromString(Filter->GetName()), FText::AsNumber(VerbosityCounters.FindRef(Filter->GetVerbosity())));
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<ITableRow> SSessionConsoleFilterBar::HandleVerbosityFilterGenerateRow(FSessionConsoleVerbosityFilterPtr Filter, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<FSessionConsoleVerbosityFilterPtr>, OwnerTable)
		[
			SNew(SCheckBox)
				.IsChecked(Filter.Get(), &FSessionConsoleVerbosityFilter::GetCheckStateFromIsEnabled)
				.Padding(FMargin(2.0f, 0.0f))
				.OnCheckStateChanged(Filter.Get(), &FSessionConsoleVerbosityFilter::EnableFromCheckState)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.0f, 0.0f)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(SImage)
								.Image(Filter->GetIcon())
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.0f, 0.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
								.Text(this, &SSessionConsoleFilterBar::HandleVerbosityFilterGetRowText, Filter)
						]
				]
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SSessionConsoleFilterBar::HandleVerbosityFilterStateChanged(ELogVerbosity::Type Verbosity, bool Enabled)
{
	if (Enabled)
	{
		DisabledVerbosities.Remove(Verbosity);
	}
	else
	{
		DisabledVerbosities.AddUnique(Verbosity);
	}

	OnFilterChanged.ExecuteIfBound();
}

FText SSessionConsoleFilterBar::GetFilterText() const
{
	return FilterStringTextBox->GetText();
}


#undef LOCTEXT_NAMESPACE
