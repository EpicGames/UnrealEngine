// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "SessionLogMessage.h"
#include "Models/SessionConsoleCategoryFilter.h"
#include "Models/SessionConsoleVerbosityFilter.h"

class SCheckBox;

/**
 * Implements the console filter bar widget.
 */
class SSessionConsoleFilterBar
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSessionConsoleFilterBar) { }

		/** Called when the filter settings have changed. */
		SLATE_EVENT(FSimpleDelegate, OnFilterChanged)

	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	void Construct(const FArguments& InArgs);

	/**
	 * Filters the specified log message based on the current filter settings.
	 *
	 * @param LogMessage The log message to filter.
	 * @return true if the log message passed the filter, false otherwise.
	 */
	bool FilterLogMessage(const TSharedRef<FSessionLogMessage>& LogMessage);

	/**
	 * Gets the current filter string.
	 *
	 * @return Filter string.
	 */
	FText GetFilterText() const;

	/** Resets the categories and filter counters. */
	void ResetFilter();

protected:

	/**
	 * Adds a category filter.
	 *
	 * @param Category The filter's category.
	 * @see AddVerbosityFilter
	 */
	void AddCategoryFilter(const FName& Category);

	/**
	 * Adds a verbosity filter.
	 *
	 * @param Verbosity The filters verbosity level.
	 * @param Name The name of the filter.
	 * @param Icon The name of the filter's icon.
	 * @see AddVerbosityFilter
	 */
	void AddVerbosityFilter(ELogVerbosity::Type Verbosity, const FString& Name, const FName& Icon);

private:

	/** Callback for generating a row widget for the category filter list. */
	TSharedRef<ITableRow> HandleCategoryFilterGenerateRow(FSessionConsoleCategoryFilterPtr Filter, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback for getting the text for a row in the category filter drop-down. */
	FText HandleCategoryFilterGetRowText(FSessionConsoleCategoryFilterPtr Filter) const;

	/** Callback for changing the enabled state of a category filter. */
	void HandleCategoryFilterStateChanged(const FName& ChangedCategory, bool Enabled);

	/** Callback for changing the filter string text box text. */
	void HandleFilterStringTextChanged(const FText& NewText);

	/** Callback for changing the checked state of the 'Filter' check box. */
	void HandleHighlightOnlyCheckBoxCheckStateChanged(ECheckBoxState CheckedState);

	/** Callback for generating a row widget for the verbosity filter list. */
	TSharedRef<ITableRow> HandleVerbosityFilterGenerateRow(FSessionConsoleVerbosityFilterPtr Filter, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback for getting the text for a row in the verbosity filter drop-down. */
	FText HandleVerbosityFilterGetRowText(FSessionConsoleVerbosityFilterPtr Filter) const;

	/** Callback for changing the check state of a filter button. */
	void HandleVerbosityFilterStateChanged(ELogVerbosity::Type Verbosity, bool Enabled);

private:

	/** Holds the list of category filters. */
	TArray<FSessionConsoleCategoryFilterPtr> CategoriesList;

	/** Holds the category filters list view. */
	TSharedPtr<SListView<FSessionConsoleCategoryFilterPtr>> CategoriesListView;

	/** Holds the log message counters for category filters. */
	TMap<FName, int32> CategoryCounters;

	/** Holds the list of disabled log categories. */
	TArray<FName> DisabledCategories;

	/** Holds the list of disabled log verbosities. */
	TArray<ELogVerbosity::Type> DisabledVerbosities;

	/** Holds the filter check box. */
	TSharedPtr<SCheckBox> HighlightOnlyCheckBox;

	/** Holds the filter string text box. */
	TSharedPtr<SSearchBox> FilterStringTextBox;

	/** Holds the verbosity filters. */
	TArray<FSessionConsoleVerbosityFilterPtr> VerbositiesList;

	/** Holds the verbosity filters list view. */
	TSharedPtr< SListView<FSessionConsoleVerbosityFilterPtr>> VerbositiesListView;

	/** Holds the log message counters for verbosity filters. */
	TMap<ELogVerbosity::Type, int32> VerbosityCounters;

private:

	/** Holds a delegate that is executed when the filter settings changed. */
	FSimpleDelegate OnFilterChanged;
};
