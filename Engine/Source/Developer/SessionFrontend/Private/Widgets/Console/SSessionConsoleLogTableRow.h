// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"
#include "Layout/Margin.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/STableRow.h"
#include "SessionLogMessage.h"
#include "SlateOptMacros.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Images/SImage.h"

class Error;

/**
 * Implements a row widget for the session console log.
 */
class SSessionConsoleLogTableRow
	: public SMultiColumnTableRow<TSharedPtr<FSessionLogMessage>>
{
public:

	SLATE_BEGIN_ARGS(SSessionConsoleLogTableRow) { }
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ARGUMENT(TSharedPtr<FSessionLogMessage>, LogMessage)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		HighlightText = InArgs._HighlightText;
		LogMessage = InArgs._LogMessage;

		SMultiColumnTableRow<TSharedPtr<FSessionLogMessage>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

public:

	// SMultiColumnTableRow interface

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == "Instance")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 1.0f, 4.0f, 0.0f))
				.HAlign(HAlign_Left)
				[
					SNew(SBorder)
						.BorderBackgroundColor(this, &SSessionConsoleLogTableRow::HandleGetBorderColor)
						.BorderImage(FAppStyle::GetBrush("ErrorReporting.Box"))
						.ColorAndOpacity(FLinearColor(0.25f, 0.25f, 0.25f))
						.Padding(FMargin(6.0f, 3.0f))
						.Content()
						[
							SNew(STextBlock)
								.Font(FAppStyle::GetFontStyle("BoldFont"))
								.Text(FText::FromString(LogMessage->InstanceName))
						]
				];
		}
		else if (ColumnName == "Message")
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(HandleGetTextColor())
						.HighlightText(HighlightText)
						.Text(FText::FromString(LogMessage->Text.Replace(TEXT("\n"), TEXT(" | ")).Replace(TEXT("\r"), TEXT(""))))
				];
		}
		else if (ColumnName == "TimeSeconds")
		{
			static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
				.SetMinimumFractionalDigits(3)
				.SetMaximumFractionalDigits(3);

			return SNew(SBox)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.0f, 0.0f, 8.0f, 0.0f))
				[
					SNew(STextBlock)
						.ColorAndOpacity(HandleGetTextColor())
						.Text(FText::AsNumber(LogMessage->TimeSeconds, &FormatOptions))
				];
		}
		else if (ColumnName == "Verbosity")
		{
			const FSlateBrush* Icon;

			if ((LogMessage->Verbosity == ELogVerbosity::Error) ||
				(LogMessage->Verbosity == ELogVerbosity::Fatal))
			{
				Icon = FAppStyle::GetBrush("Icons.Error");
			}
			else if (LogMessage->Verbosity == ELogVerbosity::Warning)
			{
				Icon = FAppStyle::GetBrush("Icons.Warning");
			}
			else
			{
				Icon = FAppStyle::GetBrush("Icons.Info");
			}

			return SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
						.Image(Icon)
				];
		}

		return SNullWidget::NullWidget;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:

	/** Gets the border color for this row. */
	FSlateColor HandleGetBorderColor() const
	{
		return FLinearColor((GetTypeHash(LogMessage->InstanceId) & 0xff) * 360.0f / 256.0f, 0.8f, 0.3f, 1.0f).HSVToLinearRGB();
	}

	/** Gets the text color for this log entry. */
	FSlateColor HandleGetTextColor() const
	{
		if ((LogMessage->Verbosity == ELogVerbosity::Error) ||
			(LogMessage->Verbosity == ELogVerbosity::Fatal))
		{
			return FLinearColor::Red;
		}
		else if (LogMessage->Verbosity == ELogVerbosity::Warning)
		{
			return FLinearColor::Yellow;
		}
		else
		{
			return FSlateColor::UseForeground();
		}
	}

private:

	/** Holds the highlight string for the log message. */
	TAttribute<FText> HighlightText;

	/** Holds a reference to the log message that is displayed in this row. */
	TSharedPtr<FSessionLogMessage> LogMessage;
};
