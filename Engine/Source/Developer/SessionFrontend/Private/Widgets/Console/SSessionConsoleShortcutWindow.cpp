// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Console/SSessionConsoleShortcutWindow.h"
#include "Misc/FileHelper.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Layout/WidgetPath.h"
#include "SlateOptMacros.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SListView.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/STextEntryPopup.h"

#define LOCTEXT_NAMESPACE "SSessionConsoleShortcutWindow"


/* SSessionConsoleShortcutWindow interface
 *****************************************************************************/

void SSessionConsoleShortcutWindow::AddShortcut(const FString& InName, const FString& InCommandString)
{
	AddShortcutInternal(InName, InCommandString);
	SaveShortcuts();
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SSessionConsoleShortcutWindow::Construct(const FArguments& InArgs)
{
	OnCommandSubmitted = InArgs._OnCommandSubmitted;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(6.0f, 0.0f, 0.0f, 0.0f)
			[
				SAssignNew(ShortcutListView, SListView<TSharedPtr<FConsoleShortcutData>>)
					.ListItemsSource(&Shortcuts)
					.SelectionMode(ESelectionMode::None)
					.OnGenerateRow(this, &SSessionConsoleShortcutWindow::HandleShortcutListViewGenerateRow)
					.HeaderRow
					(
						SNew(SHeaderRow)

						+ SHeaderRow::Column("Command")
							.DefaultLabel(LOCTEXT("ShortcutHeaderText", "Shortcuts"))
					)
			]
	];

	LoadShortcuts();
	RebuildUI();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SSessionConsoleShortcutWindow implementation
 *****************************************************************************/

void SSessionConsoleShortcutWindow::AddShortcutInternal(const FString& InName, const FString& InCommandString)
{
	TSharedPtr<FConsoleShortcutData> NewCommand = MakeShareable(new FConsoleShortcutData());
	NewCommand->Name = InName;
	NewCommand->Command = InCommandString;

	Shortcuts.Add(NewCommand);
	RebuildUI();
}


void SSessionConsoleShortcutWindow::HandleEditCommandActionExecute(TSharedPtr<FConsoleShortcutData> InShortcut, bool bInEditCommand, FText InPromptTitle)
{
	FString DefaultString = bInEditCommand ? InShortcut->Command : InShortcut->Name;

	EditedShortcut = InShortcut;
	bEditCommand = bInEditCommand;

	TSharedRef<STextEntryPopup> TextEntry = SNew(STextEntryPopup)
		.Label(InPromptTitle)
		.DefaultText(FText::FromString(*DefaultString))
		.OnTextCommitted(this, &SSessionConsoleShortcutWindow::HandleShortcutTextEntryCommitted)
		.SelectAllTextWhenFocused(true)
		.ClearKeyboardFocusOnCommit(false);

	NameEntryMenu = FSlateApplication::Get().PushMenu(
		SharedThis(this),
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
	);
}


FString SSessionConsoleShortcutWindow::GetShortcutFilename() const
{
	FString Filename = FPaths::EngineSavedDir() + TEXT("ConsoleShortcuts.txt");
	return Filename;
}


void SSessionConsoleShortcutWindow::LoadShortcuts()
{
	//clear out list of commands
	Shortcuts.Empty();

	//read file
	FString Content;
	FFileHelper::LoadFileToString(Content, *GetShortcutFilename());

	TSharedPtr<FJsonObject> ShortcutStream;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Content);
	bool bResult = FJsonSerializer::Deserialize(Reader, ShortcutStream);

	if (ShortcutStream.IsValid())
	{
		int32 CommandCount = ShortcutStream->GetNumberField(TEXT("Count"));
		for (int32 i = 0; i < CommandCount; ++i)
		{
			FString Name = ShortcutStream->GetStringField(FString::Printf(TEXT("Shortcut.%d.Name"), i));
			FString Command = ShortcutStream->GetStringField(FString::Printf(TEXT("Shortcut.%d.Command"), i));

			AddShortcut(Name, Command);
		}
	}
}


void SSessionConsoleShortcutWindow::RebuildUI()
{
	ShortcutListView->RequestListRefresh();
	SetVisibility(Shortcuts.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed);
}


void SSessionConsoleShortcutWindow::HandleDeleteCommandActionExecute(TSharedPtr<FConsoleShortcutData> InShortcut)
{
	Shortcuts.Remove(InShortcut);
	RebuildUI();
}


void SSessionConsoleShortcutWindow::SaveShortcuts() const
{
	TSharedPtr<FJsonObject> ShortcutStream = MakeShareable(new FJsonObject);

	ShortcutStream->SetNumberField(TEXT("Count"), Shortcuts.Num());
	for (int32 i = 0; i < Shortcuts.Num(); ++i)
	{
		ShortcutStream->SetStringField(FString::Printf(TEXT("Shortcut.%d.Name"), i), Shortcuts[i]->Name);
		ShortcutStream->SetStringField(FString::Printf(TEXT("Shortcut.%d.Command"), i), Shortcuts[i]->Command);
	}

	FString Content;
	TSharedRef< TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&Content);
	FJsonSerializer::Serialize(ShortcutStream.ToSharedRef(), Writer);

	FFileHelper::SaveStringToFile(*Content, *GetShortcutFilename());
}


/* SSessionConsoleShortcutWindow callbacks
 *****************************************************************************/

FReply SSessionConsoleShortcutWindow::HandleExecuteButtonClicked(TSharedPtr<FConsoleShortcutData> InShortcut)
{
	if (OnCommandSubmitted.IsBound())
	{
		OnCommandSubmitted.Execute(InShortcut->Command);
	}

	return FReply::Handled();
}


TSharedRef<ITableRow> SSessionConsoleShortcutWindow::HandleShortcutListViewGenerateRow(TSharedPtr<FConsoleShortcutData> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	FMenuBuilder ContextMenuBuilder(true, NULL);

	ContextMenuBuilder.BeginSection("SessionConsoleShortcut");
	{
		ContextMenuBuilder.AddMenuEntry(
			NSLOCTEXT("SessionFrontend", "ContextMenu.EditName", "Edit Name"),
			FText(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSessionConsoleShortcutWindow::HandleEditCommandActionExecute, InItem, false, LOCTEXT("ShortcutOptionsEditNameTitle", "Name:")))
		);

		ContextMenuBuilder.AddMenuEntry(
			NSLOCTEXT("SessionFrontend", "ContextMenu.EditCommand", "Edit Command"),
			FText(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSessionConsoleShortcutWindow::HandleEditCommandActionExecute, InItem, true, LOCTEXT("ShortcutOptionsEditCommandTitle", "Command:")))
		);
	}
	ContextMenuBuilder.EndSection();

	ContextMenuBuilder.BeginSection("SessionConsoleShortcut2");
	{
		ContextMenuBuilder.AddMenuEntry(
			NSLOCTEXT("SessionFrontend", "ContextMenu.DeleteCommand", "Delete Command"),
			FText(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSessionConsoleShortcutWindow::HandleDeleteCommandActionExecute, InItem))
		);
	}
	ContextMenuBuilder.EndSection();

	return SNew(STableRow<TSharedPtr<FConsoleShortcutData>>, OwnerTable)
		.Padding(2)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(0)
				[
					// execute button
					SNew(SButton)
						.VAlign(VAlign_Center)
						.ToolTipText(FText::FromString(InItem->Command))
						.OnClicked(FOnClicked::CreateSP(this, &SSessionConsoleShortcutWindow::HandleExecuteButtonClicked, InItem))
						[
							SNew(STextBlock)
								.Text(FText::FromString(InItem->Name))
						]
				]

			+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					// edit options pull-down
					SNew(SComboButton)
						.ButtonStyle(FAppStyle::Get(), "NoBorder")
						.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
						.ContentPadding(FMargin(6.f, 2.f))
						.MenuContent()
						[
							ContextMenuBuilder.MakeWidget()
						]
				]
		];
}


void SSessionConsoleShortcutWindow::HandleShortcutTextEntryCommitted(const FText& CommandText, ETextCommit::Type CommitInfo)
{
	if (NameEntryMenu.IsValid())
	{
		NameEntryMenu.Pin()->Dismiss();

		int32 IndexOfShortcut = Shortcuts.IndexOfByKey(EditedShortcut);

		if (EditedShortcut.IsValid() && (IndexOfShortcut != INDEX_NONE))
		{
			// make a new version of the command so the list view knows to refresh
			TSharedPtr<FConsoleShortcutData> NewCommand = MakeShareable(new FConsoleShortcutData());
			*NewCommand = *EditedShortcut;
			Shortcuts[IndexOfShortcut] = NewCommand;

			FString& StringToChange = bEditCommand ? NewCommand->Command : NewCommand->Name;
			StringToChange = CommandText.ToString();
		}

		EditedShortcut.Reset();
	}

	RebuildUI();
	SaveShortcuts();
}


#undef LOCTEXT_NAMESPACE
