// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Framework/Application/IMenu.h"
#include "Widgets/Console/SSessionConsoleCommandBar.h"

struct FConsoleShortcutData
{
	/** Name of the shortcut */
	FString Name;

	/** Command line for the shortcut */
	FString Command;
};


/**
 * Implements the console filter bar widget.
 */
class SSessionConsoleShortcutWindow
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSessionConsoleShortcutWindow) { }

		/** Called when the filter settings have changed. */
		SLATE_EVENT(FOnSessionConsoleCommandSubmitted, OnCommandSubmitted)

	SLATE_END_ARGS()

public:

	/**
	 * Adds a new shortcut to the list of commands and saves the list.
	 *
	 * @param InName The name of the shortcut to add.
	 * @param InCommandString The command associated with the shortcut.
	 */
	void AddShortcut(const FString& InName, const FString& InCommandString);

	/**
	 * Constructs this widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct(const FArguments& InArgs);

protected:

	/**
	 * Adds a new shortcut to the list of commands.
	 *
	 * @param InName The name of the shortcut.
	 * @param InCommandString The command string.
	 */
	void AddShortcutInternal(const FString& InName, const FString& InCommandString);

	/**
	 * Gets the name of the file that stores the shortcuts.
	 *
	 * @return The file name.
	 */
	FString GetShortcutFilename() const;

	/**
	 * Loads commands from save file.
	 *
	 * @see SaveShortcuts
	 */
	void LoadShortcuts();

	/** Rebuild UI or hide of there are no entries. */
	void RebuildUI();

	/**
	 * Saves commands to save file.
	 *
	 * @see LoadShortcuts
	 */
	void SaveShortcuts() const;

private:

	/** Callback for when a shortcut should be removed. */
	void HandleDeleteCommandActionExecute(TSharedPtr<FConsoleShortcutData> InShortcut);

	/** Callback for when a shortcut is executed. */
	FReply HandleExecuteButtonClicked(TSharedPtr<FConsoleShortcutData> InShortcut);

	/** Callback for when a shortcut name or command is being edited. */
	void HandleEditCommandActionExecute(TSharedPtr<FConsoleShortcutData> InShortcut, bool bInEditCommand, FText InPromptTitle);

	/** Generates a row widget for a shortcut. */
	TSharedRef<ITableRow> HandleShortcutListViewGenerateRow(TSharedPtr<FConsoleShortcutData> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback for committing changes to command. */
	void HandleShortcutTextEntryCommitted(const FText& CommandText, ETextCommit::Type CommitInfo);

private:

	/** Whether to edit the name or command. */
	bool bEditCommand;

	/** Reference to owner of the current pop-up. */
	TSharedPtr<FConsoleShortcutData> EditedShortcut;

	/** Reference to owner of the current pop-up. */
	TWeakPtr<class IMenu> NameEntryMenu;

	/** Holds a delegate that is executed when a command is submitted. */
	FOnSessionConsoleCommandSubmitted OnCommandSubmitted;

	/** List of all commands that are currently supported by shortcuts. */
	TArray<TSharedPtr<FConsoleShortcutData>> Shortcuts;

	/** The list view for showing all commands. */
	TSharedPtr<SListView<TSharedPtr<FConsoleShortcutData>>> ShortcutListView;
};
