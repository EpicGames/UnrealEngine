// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Console/SSessionConsoleToolbar.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SBorder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "Models/SessionConsoleCommands.h"


/* SSessionConsoleToolbar interface
 *****************************************************************************/

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SSessionConsoleToolbar::Construct(const FArguments& InArgs, const TSharedRef<FUICommandList>& CommandList)
{
	FSessionConsoleCommands::Register();

	// create the toolbar
	FToolBarBuilder Toolbar(CommandList, FMultiBoxCustomization::None);
	{
		Toolbar.AddToolBarButton(FSessionConsoleCommands::Get().SessionCopy);
		Toolbar.AddSeparator();
		Toolbar.AddToolBarButton(FSessionConsoleCommands::Get().Clear);
		Toolbar.AddToolBarButton(FSessionConsoleCommands::Get().SessionSave);
	}

	ChildSlot
	[
		SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(0.0f)
			[
				Toolbar.MakeWidget()
			]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION
