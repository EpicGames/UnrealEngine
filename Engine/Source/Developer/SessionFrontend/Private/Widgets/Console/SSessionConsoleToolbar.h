// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Commands/UICommandList.h"

#define LOCTEXT_NAMESPACE "SSessionConsoleToolbar"

/**
 * Implements the device toolbar widget.
 */
class SSessionConsoleToolbar
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSessionConsoleToolbar) { }
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param CommandList The command list to use.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<FUICommandList>& CommandList );
};


#undef LOCTEXT_NAMESPACE
