// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SSessionFrontend.h"

#include "ITargetDeviceProxyManager.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Textures/SlateIcon.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "ITargetDeviceServicesModule.h"
#include "IAutomationControllerModule.h"
#include "IAutomationWindowModule.h"
#include "Interfaces/IScreenShotToolsModule.h"
#include "Interfaces/IScreenShotComparisonModule.h"
#include "ISessionServicesModule.h"
#include "SessionFrontendStyle.h"
#include "TraceTools/Interfaces/ITraceToolsModule.h"
#include "Widgets/Browser/SSessionBrowser.h"
#include "Widgets/Console/SSessionConsole.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"


#define LOCTEXT_NAMESPACE "SSessionFrontend"


/* Local constants
 *****************************************************************************/

static const FName AutomationTabId("AutomationPanel");
static const FName SessionBrowserTabId("SessionBrowser");
static const FName SessionConsoleTabId("SessionConsole");
static const FName SessionScreenTabId("ScreenComparison");
static const FName TraceControlTabId("TraceControl");


/* SSessionFrontend interface
 *****************************************************************************/

void SSessionFrontend::Construct( const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow )
{
	InitializeControllers();

	// create & initialize tab manager
	TabManager = FGlobalTabmanager::Get()->NewTabManager(ConstructUnderMajorTab);
	TSharedRef<FWorkspaceItem> AppMenuGroup = TabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("SessionFrontendMenuGroupName", "Session Frontend"));
	
	TabManager->SetAllowWindowMenuBar(true);

	TabManager->RegisterTabSpawner(AutomationTabId, FOnSpawnTab::CreateRaw(this, &SSessionFrontend::HandleTabManagerSpawnTab, AutomationTabId))
		.SetDisplayName(LOCTEXT("AutomationTabTitle", "Automation"))
		.SetIcon(FSlateIcon(FSessionFrontendStyle::GetStyleSetName(), "SessionFrontEnd.Tabs.Automation"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(SessionBrowserTabId, FOnSpawnTab::CreateRaw(this, &SSessionFrontend::HandleTabManagerSpawnTab, SessionBrowserTabId))
		.SetDisplayName(LOCTEXT("SessionBrowserTitle", "Session Browser"))
		.SetIcon(FSlateIcon(FSessionFrontendStyle::GetStyleSetName(), "SessionFrontEnd.Tabs.Tools"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(SessionConsoleTabId, FOnSpawnTab::CreateRaw(this, &SSessionFrontend::HandleTabManagerSpawnTab, SessionConsoleTabId))
		.SetDisplayName(LOCTEXT("ConsoleTabTitle", "Console"))
		.SetIcon(FSlateIcon(FSessionFrontendStyle::GetStyleSetName(), "SessionFrontEnd.Tabs.Console"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(TraceControlTabId, FOnSpawnTab::CreateRaw(this, &SSessionFrontend::HandleTabManagerSpawnTab, TraceControlTabId))
		.SetDisplayName(LOCTEXT("TraceControlTabTitle", "Trace Control"))
		.SetIcon(FSlateIcon(FSessionFrontendStyle::GetStyleSetName(), "SessionFrontEnd.Tabs.TraceControl"))
		.SetGroup(AppMenuGroup);

	TabManager->RegisterTabSpawner(SessionScreenTabId, FOnSpawnTab::CreateRaw(this, &SSessionFrontend::HandleTabManagerSpawnTab, SessionScreenTabId))
		.SetDisplayName(LOCTEXT("ScreenTabTitle", "Screen Comparison"))
		.SetIcon(FSlateIcon(FSessionFrontendStyle::GetStyleSetName(), "SessionFrontEnd.Tabs.ScreenComparison"))
		.SetGroup(AppMenuGroup);

	// create tab layout
	const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("SessionFrontendLayout_v1.4")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					// session browser
					FTabManager::NewStack()
						->AddTab(SessionBrowserTabId, ETabState::OpenedTab)
						->SetHideTabWell(true)
						->SetSizeCoefficient(0.25f)
				)
				->Split
				(
					// applications
					FTabManager::NewStack()
						->AddTab(TraceControlTabId, ETabState::OpenedTab)
						->AddTab(SessionConsoleTabId, ETabState::OpenedTab)
						->AddTab(AutomationTabId, ETabState::OpenedTab)
						->AddTab(SessionScreenTabId, ETabState::OpenedTab)
						->SetSizeCoefficient(0.75f)
						->SetForegroundTab(TraceControlTabId)
				)							
		);

	// create & initialize main menu
	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(TSharedPtr<FUICommandList>());

	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("WindowMenuLabel", "Window"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateStatic(&SSessionFrontend::FillWindowMenu, TabManager),
		"Window"
	);

	TSharedRef<SWidget> MenuWidget = MenuBarBuilder.MakeWidget();

	ChildSlot
	[
		TabManager->RestoreFrom(Layout, ConstructUnderWindow).ToSharedRef()
	];

	// Tell tab-manager about the multi-box for platforms with a global menu bar
	TabManager->SetMenuMultiBox(MenuBarBuilder.GetMultiBox(), MenuWidget);
}


/* SSessionFrontend implementation
 *****************************************************************************/

void SSessionFrontend::FillWindowMenu( FMenuBuilder& MenuBuilder, const TSharedPtr<FTabManager> TabManager )
{
	if (!TabManager.IsValid())
	{
		return;
	}

#if !WITH_EDITOR
	FGlobalTabmanager::Get()->PopulateTabSpawnerMenu(MenuBuilder, WorkspaceMenu::GetMenuStructure().GetStructureRoot());
#endif //!WITH_EDITOR

	TabManager->PopulateLocalTabSpawnerMenu(MenuBuilder);
}


void SSessionFrontend::InitializeControllers()
{
	// load required modules and objects
	ISessionServicesModule& SessionServicesModule = FModuleManager::LoadModuleChecked<ISessionServicesModule>("SessionServices");
	ITargetDeviceServicesModule& TargetDeviceServicesModule = FModuleManager::LoadModuleChecked<ITargetDeviceServicesModule>("TargetDeviceServices");
	IScreenShotToolsModule& ScreenShotModule = FModuleManager::LoadModuleChecked<IScreenShotToolsModule>("ScreenShotComparisonTools");

	// create controllers
	DeviceProxyManager = TargetDeviceServicesModule.GetDeviceProxyManager();
	SessionManager = SessionServicesModule.GetSessionManager();
	ScreenShotManager = ScreenShotModule.GetScreenShotManager();
	TraceController = SessionServicesModule.GetTraceController();
}


/* SSessionFrontend callbacks
 *****************************************************************************/

void SSessionFrontend::HandleAutomationModuleShutdown()
{
	IAutomationWindowModule& AutomationWindowModule = FModuleManager::LoadModuleChecked<IAutomationWindowModule>("AutomationWindow");
	TSharedPtr<SDockTab> AutomationWindowModuleTab = AutomationWindowModule.GetAutomationWindowTab().Pin();
	if (AutomationWindowModuleTab.IsValid())
	{
		AutomationWindowModuleTab->RequestCloseTab();
	}
}


TSharedRef<SDockTab> SSessionFrontend::HandleTabManagerSpawnTab( const FSpawnTabArgs& Args, FName TabIdentifier ) const
{
	TSharedPtr<SWidget> TabWidget = SNullWidget::NullWidget;

	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab);

	if (TabIdentifier == AutomationTabId)
	{
		// create a controller every time a tab is created
		IAutomationControllerModule& AutomationControllerModule = FModuleManager::LoadModuleChecked<IAutomationControllerModule>(TEXT("AutomationController"));
		IAutomationControllerManagerPtr AutomationController = AutomationControllerModule.GetAutomationController();
		IAutomationWindowModule& AutomationWindowModule = FModuleManager::LoadModuleChecked<IAutomationWindowModule>("AutomationWindow");

		AutomationController->OnShutdown().AddSP(const_cast<SSessionFrontend*>(this), &SSessionFrontend::HandleAutomationModuleShutdown);

		TabWidget = AutomationWindowModule.CreateAutomationWindow(
			AutomationController.ToSharedRef(),
			SessionManager.ToSharedRef()
		);

		AutomationWindowModule.OnShutdown().BindSP(const_cast<SSessionFrontend*>(this), &SSessionFrontend::HandleAutomationModuleShutdown);
	}
	else if (TabIdentifier == SessionBrowserTabId)
	{
		TabWidget = SNew(SSessionBrowser, SessionManager.ToSharedRef());
	}
	else if (TabIdentifier == SessionConsoleTabId)
	{
		TabWidget = SNew(SSessionConsole, SessionManager.ToSharedRef());
	}
	else if (TabIdentifier == TraceControlTabId)
	{
		TabWidget = FModuleManager::LoadModuleChecked<UE::TraceTools::ITraceToolsModule>("TraceTools").CreateTraceControlWidget(TraceController);
	}
	else if (TabIdentifier == SessionScreenTabId)
	{
		TabWidget = FModuleManager::LoadModuleChecked<IScreenShotComparisonModule>("ScreenShotComparison").CreateScreenShotComparison(
			ScreenShotManager.ToSharedRef()
		);
	}

	DockTab->SetContent(TabWidget.ToSharedRef());

	// save the Automation Window Dock Tab so that we can close it on required module being shutdown or recompiled.
	if (TabIdentifier == AutomationTabId)
	{
		FModuleManager::LoadModuleChecked<IAutomationWindowModule>("AutomationWindow").SetAutomationWindowTab(DockTab);
	}

	return DockTab;
}


#undef LOCTEXT_NAMESPACE
