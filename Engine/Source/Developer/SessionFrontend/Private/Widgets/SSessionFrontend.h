// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "ISessionManager.h"
#include "Interfaces/IScreenShotManager.h"

class FMenuBuilder;
class FSpawnTabArgs;
class FTabManager;
class ITargetDeviceProxyManager;
class SButton;
class SWindow;

/**
 * Implements the launcher application
 */
class SSessionFrontend
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSessionFrontend) { }
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the application.
	 *
	 * @param InArgs The Slate argument list.
	 * @param ConstructUnderMajorTab The major tab which will contain the session front-end.
	 * @param ConstructUnderWindow The window in which this widget is being constructed.
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow );

	TSharedPtr<FTabManager> GetTabManager() const { return TabManager; }

protected:

	/**
	 * Fills the Window menu with menu items.
	 *
	 * @param MenuBuilder The multi-box builder that should be filled with content for this pull-down menu.
	 * @param TabManager A Tab Manager from which to populate tab spawner menu items.
	 */
	static void FillWindowMenu( FMenuBuilder& MenuBuilder, const TSharedPtr<FTabManager> TabManager );

	/** Creates and initializes the controller classes. */
	void InitializeControllers();

private:

	/** Callback for handling automation module shutdowns. */
	void HandleAutomationModuleShutdown();

	/** Callback for spawning tabs. */
	TSharedRef<SDockTab> HandleTabManagerSpawnTab( const FSpawnTabArgs& Args, FName TabIdentifier ) const;

private:

	/** Holds the target device proxy manager. */
	TSharedPtr<ITargetDeviceProxyManager> DeviceProxyManager;
	
	/** Holds a flag indicating whether the launcher overlay is visible. */
	bool LauncherOverlayVisible;

	/** Holds the 'new session' button. */
	TSharedPtr<SButton> NewSessionButton;

	/** Holds a pointer to the session manager. */
	TSharedPtr<ISessionManager> SessionManager;

	/** Holds a pointer to the session manager. */
	IScreenShotManagerPtr ScreenShotManager;

	/** Holds the tab manager that manages the front-end's tabs. */
	TSharedPtr<FTabManager> TabManager;

	/** Trace controller */
	TSharedPtr<class ITraceController> TraceController;
};
