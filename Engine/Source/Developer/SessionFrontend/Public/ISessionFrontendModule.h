// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Widgets/SWidget.h"

class ISessionManager;

/**
 * Interface for launcher UI modules.
 */
class ISessionFrontendModule
	: public IModuleInterface
{
public:

	/**
	 * Creates a session browser widget.
	 *
	 * @param SessionManager The session manager to use.
	 * @return The new session browser widget.
	 */
	virtual TSharedRef<SWidget> CreateSessionBrowser( const TSharedRef<ISessionManager>& SessionManager ) = 0;
	
	/**
	 * Creates a session console widget.
	 *
	 * @param SessionManager The session manager to use.
	 */
	virtual TSharedRef<SWidget> CreateSessionConsole( const TSharedRef<ISessionManager>& SessionManager ) = 0;

	/**
	 * Show the session frontend tab.
	 */
	virtual void InvokeSessionFrontend(FName SubTabToActivate = NAME_None) = 0;

public:

	/** Virtual destructor. */
	virtual ~ISessionFrontendModule() = default;
};
