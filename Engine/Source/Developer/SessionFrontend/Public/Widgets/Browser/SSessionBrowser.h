// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "ISessionInstanceInfo.h"
#include "ISessionInfo.h"
#include "ISessionManager.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

#define UE_API SESSIONFRONTEND_API

class FSessionBrowserGroupTreeItem;
class FSessionBrowserTreeItem;
class SSessionBrowserCommandBar;

/**
 * Implements a Slate widget for browsing active game sessions.
 */
class SSessionBrowser
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSessionBrowser) { }
	SLATE_END_ARGS()

public:

	/** Destructor. */
	UE_API ~SSessionBrowser();

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InSessionManager The session manager to use.
	 */
	UE_API void Construct( const FArguments& InArgs, TSharedRef<ISessionManager> InSessionManager );

protected:

	/**
	 * Fully expands the specified tree view item.
	 *
	 * @param Item The tree view item to expand.
	 */
	UE_API void ExpandItem(const TSharedPtr<FSessionBrowserTreeItem>& Item);

	/** Filters the session tree. */
	UE_API void FilterSessions();

	/** Adds items for this session in the tree. */
	UE_API void AddInstanceItemToTree(TSharedPtr<FSessionBrowserTreeItem>& SessionItem, const TSharedPtr<FSessionBrowserTreeItem>& InstanceItem, const TSharedPtr<ISessionInstanceInfo>& InstanceInfo);

	/** Reloads the sessions list. */
	UE_API void ReloadSessions();

private:

	/** Callback for changing the selection state of an instance. */
	UE_API void HandleSessionManagerInstanceSelectionChanged(const TSharedPtr<ISessionInstanceInfo>& Instance, bool Selected);

	/** Callback for changing the selected session in the session manager. */
	UE_API void HandleSessionManagerSelectedSessionChanged(const TSharedPtr<ISessionInfo>& SelectedSession);

	/** Callback for updating the session list in the session manager. */
	UE_API void HandleSessionManagerSessionsUpdated();

	/** Callback from the session manager to notify there's a new session instance. */
	UE_API void HandleSessionManagerInstanceDiscovered(const TSharedRef<ISessionInfo>& OwnerSession, const TSharedRef<ISessionInstanceInfo>& DiscoveredInstance);

	/** Callback for getting the tool tip text of a session tree row. */
	UE_API FText HandleSessionTreeRowGetToolTipText(TSharedPtr<FSessionBrowserTreeItem> Item) const;

	/** Callback for session tree view selection changes. */
	UE_API void HandleSessionTreeViewExpansionChanged(TSharedPtr<FSessionBrowserTreeItem> TreeItem, bool bIsExpanded);

	/** Callback for generating a row widget in the session tree view. */
	UE_API TSharedRef<ITableRow> HandleSessionTreeViewGenerateRow(TSharedPtr<FSessionBrowserTreeItem> Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback for getting the children of a node in the session tree view. */
	UE_API void HandleSessionTreeViewGetChildren(TSharedPtr<FSessionBrowserTreeItem> Item, TArray<TSharedPtr<FSessionBrowserTreeItem>>& OutChildren);

	/** Callback for session tree view selection changes. */
	UE_API void HandleSessionTreeViewSelectionChanged(const TSharedPtr<FSessionBrowserTreeItem> Item, ESelectInfo::Type SelectInfo);

	/** Callback for clicking the 'Terminate Session' button. */
	UE_API FReply HandleTerminateSessionButtonClicked();

	/** Callback for getting the enabled state of the 'Terminate Session' button. */
	UE_API bool HandleTerminateSessionButtonIsEnabled() const;

private:

	/** Holds an unfiltered list of available sessions. */
	TArray<TSharedPtr<ISessionInfo>> AvailableSessions;

	/** Holds the command bar. */
	TSharedPtr<SSessionBrowserCommandBar> CommandBar;

	/** Whether to ignore events from the session manager. */
	bool IgnoreSessionManagerEvents;

	/** Whether to ignore events from the session tree view. */
	bool updatingTreeExpansion;

	/** Maps session and instance GUIDs to existing tree items. */
	TMap<FGuid, TSharedPtr<FSessionBrowserTreeItem>> ItemMap;

	/** Holds a reference to the session manager. */
	TSharedPtr<ISessionManager> SessionManager;

	/** Holds the filtered list of tree items. */
	TArray<TSharedPtr<FSessionBrowserTreeItem>> SessionTreeItems;

	/** Holds the session tree view. */
	TSharedPtr<STreeView<TSharedPtr<FSessionBrowserTreeItem>>> SessionTreeView;

private:

	/** The session tree item that holds this application's session. */
	TSharedPtr<FSessionBrowserGroupTreeItem> AppGroupItem;

	/** The session tree item that holds other user's sessions. */
	TSharedPtr<FSessionBrowserGroupTreeItem> OtherGroupItem;

	/** The session tree item that holds the owner's remote sessions. */
	TSharedPtr<FSessionBrowserGroupTreeItem> OwnerGroupItem;

	/** The session tree item that holds other user's standalone instances. */
	TSharedPtr<FSessionBrowserGroupTreeItem> StandaloneGroupItem;

	/** This app's instance session */
	TWeakPtr<FSessionBrowserTreeItem> ThisAppInstance;

	/** True if we should set the default selection the next time the tree view if refreshed. */
	bool bCanSetDefaultSelection;
};

#undef UE_API
