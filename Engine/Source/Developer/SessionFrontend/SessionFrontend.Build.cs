// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using UnrealBuildTool;

public class SessionFrontend : ModuleRules
{
	public SessionFrontend(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Slate",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"DesktopPlatform",
				"ApplicationCore",
				"InputCore",
				"Json",
				"SessionServices",
				"SlateCore",

				// @todo gmp: remove these dependencies by making the session front-end extensible
				"AutomationWindow",
				"ScreenShotComparison",
				"ScreenShotComparisonTools",
				"TargetPlatform",
				"TraceTools",
				"WorkspaceMenuStructure",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Messaging",
				"TargetDeviceServices",
			}
		);
	}
}
